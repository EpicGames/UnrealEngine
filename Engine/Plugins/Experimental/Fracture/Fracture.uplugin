{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Fracture",
	"Description": "Adds Module for FractureEditor",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "FractureEngine",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "PlanarCut",
			"Enabled": true
		}
	]
}



