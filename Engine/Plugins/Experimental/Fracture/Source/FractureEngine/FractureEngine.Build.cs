// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class FractureEngine : ModuleRules
	{
		public FractureEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DataflowCore",
					"PlanarCut",
					"GeometryCore",
					"DynamicMesh",
					"Voronoi",
					"MeshUtilitiesCommon", // for FDisjointSet
				}
				);
			SetupModulePhysicsSupport(Target);
		}
	}
}

