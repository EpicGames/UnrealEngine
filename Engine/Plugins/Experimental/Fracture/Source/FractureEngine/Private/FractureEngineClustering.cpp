// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineClustering.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Math/BoxSphereBounds.h"
#include "VertexConnectedComponents.h"
#include "CompGeom/ConvexDecomposition3.h"
#include "CompGeom/ConvexHull3.h"

namespace UE::Private::ClusterMagnet
{
	struct FClusterMagnet
	{
		TSet<int32> ClusteredNodes;
		TSet<int32> Connections;
	};

	TMap<int32, TSet<int32>> InitializeConnectivity(const TSet<int32>& TopNodes, FGeometryCollection* GeometryCollection, int32 OperatingLevel);
	void CollectTopNodeConnections(FGeometryCollection* GeometryCollection, int32 Index, int32 OperatingLevel, TSet<int32>& OutConnections);
	void SeparateClusterMagnets(const TSet<int32>& TopNodes, const TArray<int32>& Selection, const TMap<int32, TSet<int32>>& TopNodeConnectivity, TArray<FClusterMagnet>& OutClusterMagnets, TSet<int32>& OutRemainingPool);
	bool AbsorbClusterNeighbors(const TMap<int32, TSet<int32>> TopNodeConnectivity, FClusterMagnet& OutClusterMagnets, TSet<int32>& OutRemainingPool);


	TMap<int32, TSet<int32>> InitializeConnectivity(const TSet<int32>& TopNodes, FGeometryCollection* GeometryCollection, int32 OperatingLevel)
	{
		FGeometryCollectionProximityUtility ProximityUtility(GeometryCollection);
		ProximityUtility.RequireProximity();

		TMap<int32, TSet<int32>> ConnectivityMap;
		for (int32 Index : TopNodes)
		{
			// Collect the proximity indices of all the leaf nodes under this top node,
			// traced back up to its parent top node, so that all connectivity describes
			// relationships only between top nodes.
			TSet<int32> Connections;
			CollectTopNodeConnections(GeometryCollection, Index, OperatingLevel, Connections);
			Connections.Remove(Index);

			// Remove any connections outside the current operating branch.
			ConnectivityMap.Add(Index, Connections.Intersect(TopNodes));
		}

		return ConnectivityMap;
	}

	void CollectTopNodeConnections(FGeometryCollection* GeometryCollection, int32 Index, int32 OperatingLevel, TSet<int32>& OutConnections)
	{
		const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollection->TransformToGeometryIndex;
		if (GeometryCollection->SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_Rigid
			&& TransformToGeometryIndex[Index] != INDEX_NONE) // rigid node with geometry, leaf of the simulated part
		{
			const TManagedArray<TSet<int32>>& Proximity = GeometryCollection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
			const TManagedArray<int32>& GeometryToTransformIndex = GeometryCollection->TransformIndex;


			for (int32 Neighbor : Proximity[TransformToGeometryIndex[Index]])
			{
				int32 NeighborTransformIndex = GeometryToTransformIndex[Neighbor];
				OutConnections.Add(FGeometryCollectionClusteringUtility::GetParentOfBoneAtSpecifiedLevel(GeometryCollection, NeighborTransformIndex, OperatingLevel));
			}
		}
		else
		{
			const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
			for (int32 ChildIndex : Children[Index])
			{
				CollectTopNodeConnections(GeometryCollection, ChildIndex, OperatingLevel, OutConnections);
			}
		}
	}

	void SeparateClusterMagnets(
		const TSet<int32>& TopNodes,
		const TArray<int32>& Selection,
		const TMap<int32, TSet<int32>>& TopNodeConnectivity,
		TArray<FClusterMagnet>& OutClusterMagnets,
		TSet<int32>& OutRemainingPool)
	{
		OutClusterMagnets.Reserve(TopNodes.Num());
		OutRemainingPool.Reserve(TopNodes.Num());

		for (int32 Index : TopNodes)
		{
			if (Selection.Contains(Index))
			{
				OutClusterMagnets.AddDefaulted();
				FClusterMagnet& NewMagnet = OutClusterMagnets.Last();
				NewMagnet.ClusteredNodes.Add(Index);
				NewMagnet.Connections = TopNodeConnectivity[Index];
			}
			else
			{
				OutRemainingPool.Add(Index);
			}
		}
	}

	bool AbsorbClusterNeighbors(const TMap<int32, TSet<int32>> TopNodeConnectivity, FClusterMagnet& OutClusterMagnet, TSet<int32>& OutRemainingPool)
	{
		// Return true if neighbors were absorbed.
		bool bNeighborsAbsorbed = false;

		TSet<int32> NewConnections;
		for (int32 NeighborIndex : OutClusterMagnet.Connections)
		{
			// If the neighbor is still in the pool, absorb it and its connections.
			if (OutRemainingPool.Contains(NeighborIndex))
			{
				OutClusterMagnet.ClusteredNodes.Add(NeighborIndex);
				NewConnections.Append(TopNodeConnectivity[NeighborIndex]);
				OutRemainingPool.Remove(NeighborIndex);
				bNeighborsAbsorbed = true;
			}
		}
		OutClusterMagnet.Connections.Append(NewConnections);

		return bNeighborsAbsorbed;
	}
}


FVoronoiPartitioner::FVoronoiPartitioner(const FGeometryCollection* GeometryCollection, int32 ClusterIndex)
{
	// Collect children of cluster
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	TransformIndices = Children[ClusterIndex].Array();

	GenerateCentroids(GeometryCollection);
}

void FVoronoiPartitioner::KMeansPartition(int32 InPartitionCount, int32 MaxIterations, TArrayView<const FVector> InitialCenters)
{
	PartitionCount = InPartitionCount;
	if (InitialCenters.Num())
	{
		PartitionCount = InitialCenters.Num();
	}

	InitializePartitions(InitialCenters);
	// Note: We run max iterations + 1 here because the first iteration just applies the initial cluster assignment
	for (int32 Iteration = 0; Iteration < MaxIterations + 1; Iteration++)
	{
		// Refinement is complete when no nodes change partition.
		if (!Refine())
		{
			break;
		}
	}
}

void FVoronoiPartitioner::MergeSingleElementPartitions(FGeometryCollection* GeometryCollection)
{
	if (Connectivity.IsEmpty())
	{
		GenerateConnectivity(GeometryCollection);
	}
	for (int32 ElIdx = 0; ElIdx < TransformIndices.Num(); ElIdx++)
	{
		int32 Partition = Partitions[ElIdx];
		if (PartitionSize[Partition] == 1)
		{
			// Find the smallest neighboring partition to merge to
			// (to help keep partition sizes balanced)
			int32 SmallestNbrPartition = -1;
			int32 SmallestNbrSize = TransformIndices.Num()+1;
			for (int32 NbrEl : Connectivity[ElIdx])
			{
				int32 NbrPartition = Partitions[NbrEl];
				if (NbrPartition == Partition)
				{
					continue;
				}
				int32 NbrSize = PartitionSize[NbrPartition];
				if (NbrSize > 0 && (SmallestNbrPartition == -1 || NbrSize < SmallestNbrSize))
				{
					SmallestNbrPartition = NbrPartition;
					SmallestNbrSize = NbrSize;
				}
			}
			if (SmallestNbrPartition != -1)
			{
				Partitions[ElIdx] = SmallestNbrPartition;
				PartitionSize[Partition]--;
				PartitionSize[SmallestNbrPartition]++;
			}
		}
	}
}

void FVoronoiPartitioner::MergeSmallPartitions(FGeometryCollection* GeometryCollection, float SizeThreshold)
{
	if (Connectivity.IsEmpty())
	{
		GenerateConnectivity(GeometryCollection);
	}

	FGeometryCollectionConvexUtility::SetVolumeAttributes(GeometryCollection);
	const TManagedArray<float>& Volumes = GeometryCollection->GetAttribute<float>("Volume", FTransformCollection::TransformGroup);

	float VolumeThreshold = SizeThreshold * SizeThreshold * SizeThreshold;
	TArray<float> PartitionVolumes;
	PartitionVolumes.Init(0, PartitionSize.Num());
	
	int32 NonEmptyPartitions = GetNonEmptyPartitionCount();

	for (int32 ElIdx = 0; ElIdx < TransformIndices.Num(); ElIdx++)
	{
		int32 Partition = Partitions[ElIdx];
		PartitionVolumes[Partition] += Volumes[TransformIndices[ElIdx]];
	}

	TArray<int32> ToConsider;
	ToConsider.Reserve(Partitions.Num());
	for (int32 Partition = 0; Partition < PartitionVolumes.Num(); ++Partition)
	{
		if (PartitionVolumes[Partition] < VolumeThreshold)
		{
			ToConsider.Emplace(Partition);
		}
	}
	while (!ToConsider.IsEmpty())
	{
		int32 Partition = ToConsider.Pop(EAllowShrinking::No);
		if (NonEmptyPartitions < 3)
		{
			break; // if we only have two partitions, stop merging
		}
		if (PartitionVolumes[Partition] < VolumeThreshold)
		{
			int32 SmallestNbrPartition = INDEX_NONE;
			float SmallestNbrVolume = FMathf::MaxReal;
			for (int32 ElIdx = 0; ElIdx < TransformIndices.Num(); ++ElIdx)
			{
				if (Partitions[ElIdx] == Partition)
				{
					for (int32 NbrEl : Connectivity[ElIdx])
					{
						int32 NbrPartition = Partitions[NbrEl];
						if (NbrPartition != Partition)
						{
							if (SmallestNbrPartition == INDEX_NONE || PartitionVolumes[NbrPartition] < SmallestNbrVolume)
							{
								SmallestNbrVolume = PartitionVolumes[NbrPartition];
								SmallestNbrPartition = NbrPartition;
							}
						}
					}
				}
			}
			if (SmallestNbrPartition != INDEX_NONE)
			{
				PartitionVolumes[SmallestNbrPartition] += PartitionVolumes[Partition];
				PartitionVolumes[Partition] = 0;
				float CombinedSize = PartitionSize[SmallestNbrPartition] + PartitionSize[Partition];
				PartitionSize[SmallestNbrPartition] = CombinedSize;
				if (CombinedSize < VolumeThreshold)
				{
					ToConsider.Push(SmallestNbrPartition);
					Swap(ToConsider.Last(), ToConsider[0]); // make sure we don't always merge to the same cluster
				}
				PartitionSize[Partition] = 0;
				NonEmptyPartitions--;
				for (int32 ElIdx = 0; ElIdx < TransformIndices.Num(); ++ElIdx)
				{
					if (Partitions[ElIdx] == Partition)
					{
						Partitions[ElIdx] = SmallestNbrPartition;
					}
				}
			}
		}
	}
}

void FVoronoiPartitioner::SplitDisconnectedPartitions(FGeometryCollection* GeometryCollection)
{
	Visited.Init(false, TransformIndices.Num());
	if (Connectivity.IsEmpty())
	{
		GenerateConnectivity(GeometryCollection);
	}
	
	for (int32 PartitionIndex = 0; PartitionIndex < PartitionCount; ++PartitionIndex)
	{
		int32 FirstElementInPartitionIndex = Partitions.Find(PartitionIndex);
		if (FirstElementInPartitionIndex > INDEX_NONE)
		{
			MarkVisited(FirstElementInPartitionIndex, PartitionIndex);

			// Place unvisited partition members in a new partition.
			bool bFoundUnattached = false;
			for (int32 Index = 0; Index < TransformIndices.Num(); ++Index)
			{
				if (Partitions[Index] == PartitionIndex)
				{
					if (!Visited[Index])
					{
						if (!bFoundUnattached)
						{
							bFoundUnattached = true;
							PartitionCount++;
							PartitionSize.Add(0);
						}
						Partitions[Index] = PartitionCount - 1;
						PartitionSize[PartitionCount - 1]++;
						PartitionSize[PartitionIndex]--;
					}
				}
			}
		}
	}
}

void FVoronoiPartitioner::MarkVisited(int32 Index, int32 PartitionIndex)
{
	Visited[Index] = true;
	for (int32 Adjacent : Connectivity[Index])
	{
		if (!Visited[Adjacent] && Partitions[Adjacent] == PartitionIndex)
		{
			MarkVisited(Adjacent, PartitionIndex);
		}
	}
}

TArray<int32> FVoronoiPartitioner::GetPartition(int32 PartitionIndex) const
{
	TArray<int32> PartitionContents;
	PartitionContents.Reserve(TransformIndices.Num());
	for (int32 Index = 0; Index < TransformIndices.Num(); ++Index)
	{
		if (Partitions[Index] == PartitionIndex)
		{
			PartitionContents.Add(TransformIndices[Index]);
		}
	}

	return PartitionContents;
}

void FVoronoiPartitioner::GenerateConnectivity(const FGeometryCollection* GeometryCollection)
{
	
	Connectivity.SetNum(TransformIndices.Num());

	if (!ensure(GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup)))
	{
		return;
	}
	const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

	for (int32 Index = 0; Index < TransformIndices.Num(); ++Index)
	{
		CollectConnections(GeometryCollection, TransformIndices[Index], Levels[TransformIndices[Index]], Connectivity[Index]);
	}
}

void FVoronoiPartitioner::CollectConnections(const FGeometryCollection* GeometryCollection, int32 Index, int32 OperatingLevel, TSet<int32>& OutConnections) const
{
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	if (GeometryCollection->IsRigid(Index))
	{
		const TManagedArray<TSet<int32>>& Proximity = GeometryCollection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& GeometryToTransformIndex = GeometryCollection->TransformIndex;
		const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollection->TransformToGeometryIndex;

		for (int32 Neighbor : Proximity[TransformToGeometryIndex[Index]])
		{
			int32 NeighborTransformIndex = FGeometryCollectionClusteringUtility::GetParentOfBoneAtSpecifiedLevel(GeometryCollection, GeometryToTransformIndex[Neighbor], OperatingLevel);
			int32 ContextIndex = TransformIndices.Find(NeighborTransformIndex);
			if (ContextIndex > INDEX_NONE)
			{
				OutConnections.Add(ContextIndex);
			}
		}
	}
	else
	{
		for (int32 ChildIndex : Children[Index])
		{
			CollectConnections(GeometryCollection, ChildIndex, OperatingLevel, OutConnections);
		}
	}
}

void FVoronoiPartitioner::GenerateCentroids(const FGeometryCollection* GeometryCollection)
{
	Centroids.SetNum(TransformIndices.Num());
	ParallelFor(TransformIndices.Num(), [this, GeometryCollection](int32 Index)
		{
			Centroids[Index] = GenerateCentroid(GeometryCollection, TransformIndices[Index]);
		});
}

FVector FVoronoiPartitioner::GenerateCentroid(const FGeometryCollection* GeometryCollection, int32 TransformIndex) const
{
	FBox Bounds = GenerateBounds(GeometryCollection, TransformIndex);
	return Bounds.GetCenter();
}

FBox FVoronoiPartitioner::GenerateBounds(const FGeometryCollection* GeometryCollection, int32 TransformIndex)
{
	check(GeometryCollection->IsRigid(TransformIndex) || GeometryCollection->IsClustered(TransformIndex));

	// Return the bounds of all the vertices contained by this branch.

	if (GeometryCollection->IsRigid(TransformIndex))
	{
		const TManagedArray<FTransform3f>& Transforms = GeometryCollection->Transform;
		const TManagedArray<int32>& Parents = GeometryCollection->Parent;
		FTransform3f GlobalTransform = GeometryCollectionAlgo::GlobalMatrix3f(Transforms, Parents, TransformIndex);
		
		int32 GeometryIndex = GeometryCollection->TransformToGeometryIndex[TransformIndex];
		int32 VertexStart = GeometryCollection->VertexStart[GeometryIndex];
		int32 VertexCount = GeometryCollection->VertexCount[GeometryIndex];
		const TManagedArray<FVector3f>& GCVertices = GeometryCollection->Vertex;

		TArray<FVector> Vertices;
		Vertices.SetNum(VertexCount);
		for (int32 VertexOffset = 0; VertexOffset < VertexCount; ++VertexOffset)
		{
			Vertices[VertexOffset] = FVector(GlobalTransform.TransformPosition(GCVertices[VertexStart + VertexOffset]));
		}

		return FBox(Vertices);
	}
	else // recurse the cluster
	{
		const TArray<int32> Children = GeometryCollection->Children[TransformIndex].Array();

		// Empty clusters are problematic
		if (Children.Num() == 0)
		{
			return FBox();
		}

		FBox Bounds = GenerateBounds(GeometryCollection, Children[0]);
		for (int32 ChildIndex = 1; ChildIndex < Children.Num(); ++ChildIndex)
		{
			Bounds += GenerateBounds(GeometryCollection, Children[ChildIndex]);
		}

		return Bounds;
	}
}

void FVoronoiPartitioner::InitializePartitions(TArrayView<const FVector> InitialCenters)
{
	check(PartitionCount > 0);

	if (InitialCenters.Num() > 0)
	{
		PartitionCenters.Reset();
		PartitionCenters.Append(InitialCenters);
	}
	else
	{
		PartitionCount = FMath::Min(PartitionCount, TransformIndices.Num());

		// Set initial partition centers as selects from the vertex set
		PartitionCenters.SetNum(PartitionCount);
		int32 TransformStride = FMath::Max(1, FMath::FloorToInt(float(TransformIndices.Num()) / float(PartitionCount)));
		for (int32 PartitionIndex = 0; PartitionIndex < PartitionCount; ++PartitionIndex)
		{
			PartitionCenters[PartitionIndex] = Centroids[TransformStride * PartitionIndex];
		}
	}

	// At beginning, all nodes belong to first partition.
	Partitions.Init(0, TransformIndices.Num());
	PartitionSize.Init(0, PartitionCount);

	if (PartitionCount > 0)
	{
		PartitionSize[0] = TransformIndices.Num();
	}
}

bool FVoronoiPartitioner::Refine()
{
	// Assign each transform to its closest partition center.
	bool bUnchanged = true;
	for (int32 Index = 0; Index < Centroids.Num(); ++Index)
	{
		int32 ClosestPartition = FindClosestPartitionCenter(Centroids[Index]);
		if (ClosestPartition != Partitions[Index])
		{
			bUnchanged = false;
			PartitionSize[Partitions[Index]]--;
			PartitionSize[ClosestPartition]++;
			Partitions[Index] = ClosestPartition;
		}
	}

	if (bUnchanged)
	{
		return false;
	}

	// Recalculate partition centers
	for (int32 PartitionIndex = 0; PartitionIndex < PartitionCount; ++PartitionIndex)
	{
		if (PartitionSize[PartitionIndex] > 0)
		{
			PartitionCenters[PartitionIndex] = FVector(0.0f, 0.0f, 0.0f);
		}
	}

	for (int32 Index = 0; Index < Partitions.Num(); ++Index)
	{
		PartitionCenters[Partitions[Index]] += Centroids[Index];
	}

	for (int32 PartitionIndex = 0; PartitionIndex < PartitionCount; ++PartitionIndex)
	{
		if (PartitionSize[PartitionIndex] > 0)
		{
			PartitionCenters[PartitionIndex] /= float(PartitionSize[PartitionIndex]);
		}
	}

	return true;
}

int32 FVoronoiPartitioner::FindClosestPartitionCenter(const FVector& Location) const
{
	int32 ClosestPartition = INDEX_NONE;
	float SmallestDistSquared = TNumericLimits<float>::Max();

	for (int32 PartitionIndex = 0; PartitionIndex < PartitionCount; ++PartitionIndex)
	{
		float DistSquared = FVector::DistSquared(Location, PartitionCenters[PartitionIndex]);
		if (DistSquared < SmallestDistSquared)
		{
			SmallestDistSquared = DistSquared;
			ClosestPartition = PartitionIndex;
		}
	}

	check(ClosestPartition > INDEX_NONE);
	return ClosestPartition;
}

class FFractureEngineBoneSelection
{
public:
	FFractureEngineBoneSelection(const FGeometryCollection& InGeometryCollection, const TArray<int32>& InSelectedBones)
		: GeometryCollection(InGeometryCollection)
		, SelectedBones(InSelectedBones)
	{}

	const TArray<int32>& GetSelectedBones() const { return SelectedBones;  }
	void ConvertSelectionToClusterNodes();

private:
	bool IsValidBone(int32 Index) const;
	bool HasSelectedAncestor(int32 Index) const;
	void Sanitize(bool bFavorParents = true);

private:
	const FGeometryCollection& GeometryCollection;
	TArray<int32> SelectedBones;
};

bool FFractureEngineBoneSelection::IsValidBone(int32 Index) const
{
	return Index >= 0 && Index < GeometryCollection.Parent.Num();
}

bool FFractureEngineBoneSelection::HasSelectedAncestor(int32 Index) const
{
	const TManagedArray<int32>& Parents = GeometryCollection.GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

	if (!ensureMsgf(Index >= 0 && Index < Parents.Num(), TEXT("Invalid index in selection: %d"), Index))
	{
		return false;
	}

	int32 CurrIndex = Index;
	while (Parents[CurrIndex] != INDEX_NONE)
	{
		CurrIndex = Parents[CurrIndex];
		if (SelectedBones.Contains(CurrIndex))
		{
			return true;
		}
	}

	// We've arrived at the top of the hierarchy with no selected ancestors
	return false;
}

void FFractureEngineBoneSelection::Sanitize(bool bFavorParents)
{
	// Ensure that selected indices are valid
	int NumTransforms = GeometryCollection.NumElements(FGeometryCollection::TransformGroup);
	SelectedBones.RemoveAll([NumTransforms](int32 Index) {
		return Index == INDEX_NONE || !ensure(Index < NumTransforms);
	});

	// Ensure that children of a selected node are not also selected.
	if (bFavorParents)
	{
		SelectedBones.RemoveAll([this](int32 Index) {
			return !IsValidBone(Index) || HasSelectedAncestor(Index);
		});
	}

	SelectedBones.Sort();
}

void FFractureEngineBoneSelection::ConvertSelectionToClusterNodes()
{
	// If this is a non-cluster node, select the cluster containing it instead.
	const TManagedArray<TSet<int32>>& Children = GeometryCollection.Children;
	const TManagedArray<int32>& Parents = GeometryCollection.Parent;
	const TManagedArray<int32>& SimulationType = GeometryCollection.SimulationType;

	Sanitize();

	TArray<int32> AddedClusterSelections;
	for (int32 Index : SelectedBones)
	{
		int32 AddParent = FGeometryCollection::Invalid;
		if (SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			AddParent = Parents[Index];
		}
		else if (SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_None)
		{
			AddParent = Parents[Parents[Index]];
		}
		if (AddParent != FGeometryCollection::Invalid)
		{
			AddedClusterSelections.AddUnique(AddParent);
		}
	}
	SelectedBones.Append(AddedClusterSelections);

	Sanitize();
}

void FFractureEngineClustering::AutoCluster(FGeometryCollection& GeometryCollection,
	const TArray<int32>& BoneIndices,
	const EFractureEngineClusterSizeMethod ClusterSizeMethod,
	const uint32 SiteCount,
	const float SiteCountFraction,
	const float SiteSize,
	const bool bEnforceConnectivity,
	const bool bAvoidIsolated,
	const bool bEnforceSiteParameters,
	const int32 GridX,
	const int32 GridY,
	const int32 GridZ,
	const float MinimumClusterSize,
	const int32 KMeansIterations,
	const bool bPreferConvexity,
	const float ConcavityTolerance)
{
	FFractureEngineBoneSelection Selection(GeometryCollection, BoneIndices);

	Selection.ConvertSelectionToClusterNodes();

	for (const int32 ClusterIndex : Selection.GetSelectedBones())
	{
		AutoCluster(GeometryCollection, ClusterIndex, ClusterSizeMethod, SiteCount, SiteCountFraction, SiteSize, 
			bEnforceConnectivity, bAvoidIsolated, bEnforceSiteParameters, GridX, GridY, GridZ, MinimumClusterSize, 
			KMeansIterations, bPreferConvexity, ConcavityTolerance);
	}
}

void FFractureEngineClustering::ConvexityBasedCluster(FGeometryCollection& GeometryCollection,
	int32 ClusterIndex,
	uint32 SiteCount,
	bool bEnforceConnectivity,
	bool bAvoidIsolated,
	float ConcavityTolerance
)
{
	const bool bUseVolumesOfConvexHulls = true;
	if (bUseVolumesOfConvexHulls)
	{
		FGeometryCollectionConvexUtility::SetVolumeAttributes(&GeometryCollection);
	}
	FGeometryCollectionProximityUtility ProximityUtility(&GeometryCollection);
	ProximityUtility.UpdateProximity();
	const TManagedArray<float>& GeometryVolumes = GeometryCollection.GetAttribute<float>("Volume", FTransformCollection::TransformGroup);


	UE::Geometry::FConvexDecomposition3 ConvexDecomp;
	const TSet<int32>& ChildrenSet = GeometryCollection.Children[ClusterIndex];
	const TArray<int32> Children = ChildrenSet.Array();
	UE::Geometry::FSizedDisjointSet Components(Children.Num());

	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(GeometryCollection);
	HierarchyFacade.GenerateLevelAttribute();
	const TManagedArray<int32>& Levels = GeometryCollection.GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	int32 Level = Levels[ClusterIndex];

	// use source geometry as input convex vertices
	TArray<int32> TransformVertexCounts;
	TArray<int32> TransformVertexStarts;
	TransformVertexCounts.SetNumUninitialized(Children.Num());
	TransformVertexStarts.SetNumUninitialized(Children.Num());
	TArray<double> Volumes;
	Volumes.SetNumZeroed(Children.Num());
	TArray<FVector3d> GlobalVertices;
	TArray<int32> ToProcess;
	TArray<FTransform> GlobalTransforms;

	if (!bUseVolumesOfConvexHulls)
	{
		for (int32 ChildIdx = 0; ChildIdx < Children.Num(); ++ChildIdx)
		{
			Volumes[ChildIdx] = GeometryVolumes[Children[ChildIdx]];
		}
	}

	GeometryCollectionAlgo::GlobalMatrices(GeometryCollection.Transform, GeometryCollection.Parent, GlobalTransforms);
	for (int32 ChildIdx = 0; ChildIdx < Children.Num(); ++ChildIdx)
	{
		TransformVertexStarts[ChildIdx] = GlobalVertices.Num();
		int32 TransformIdx = Children[ChildIdx];
		ToProcess.Reset();
		ToProcess.Add(TransformIdx);
		double Volume = 0;
		while (!ToProcess.IsEmpty())
		{
			int32 ProcessTransformIdx = ToProcess.Pop(EAllowShrinking::No);
			FGeometryCollection::ESimulationTypes SimType = (FGeometryCollection::ESimulationTypes)GeometryCollection.SimulationType[ProcessTransformIdx];
			if (SimType == FGeometryCollection::ESimulationTypes::FST_Clustered)
			{
				for (int32 ProcChild : GeometryCollection.Children[ProcessTransformIdx])
				{
					ToProcess.Add(ProcChild);
				}
			}
			else if (SimType == FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				int32 ProcessGeometryIdx = GeometryCollection.TransformToGeometryIndex[ProcessTransformIdx];
				if (ProcessGeometryIdx == INDEX_NONE)
				{
					continue;
				}
				int32 GCVStart = GeometryCollection.VertexStart[ProcessGeometryIdx];
				int32 GCVEnd = GCVStart + GeometryCollection.VertexCount[ProcessGeometryIdx];
				int32 GlobalStart = GlobalVertices.Num();
				for (int32 VIdx = GCVStart; VIdx < GCVEnd; ++VIdx)
				{
					GlobalVertices.Add(GlobalTransforms[ProcessTransformIdx].TransformPosition((FVector3d)GeometryCollection.Vertex[VIdx]));
				}
				if (bUseVolumesOfConvexHulls)
				{
					double HullVolume = UE::Geometry::FConvexHull3d::ComputeVolume(TArrayView<const FVector3d>(&GlobalVertices[GlobalStart], GlobalVertices.Num() - GlobalStart));
					Volumes[ChildIdx] += HullVolume;
				}
			}
		}
		TransformVertexCounts[ChildIdx] = GlobalVertices.Num() - TransformVertexStarts[ChildIdx];
	}

	TArray<TPair<int32, int32>> ChildrenProximity;
	if (bEnforceConnectivity)
	{
		TMap<int32, TSet<int32>> Connectivity = UE::Private::ClusterMagnet::InitializeConnectivity(ChildrenSet, &GeometryCollection, Level + 1);
		TMap<int32, int32> TransformIdxToChildrenIdx;
		for (int32 LocalIdx = 0; LocalIdx < Children.Num(); ++LocalIdx)
		{
			TransformIdxToChildrenIdx.Add(Children[LocalIdx], LocalIdx);
		}
		for (const TPair<int32, TSet<int32>>& ConnectionsA : Connectivity)
		{
			int32 ChildIdxA = TransformIdxToChildrenIdx[ConnectionsA.Key];
			const TSet<int32>& ConnectedTo = ConnectionsA.Value;
			for (int32 TransformB : ConnectedTo)
			{
				int32 ChildIdxB = TransformIdxToChildrenIdx[TransformB];
				ChildrenProximity.Emplace(ChildIdxA, ChildIdxB);
			}
		}
	}

	ConvexDecomp.InitializeFromHulls(Children.Num(),
		[&](int32 Idx)->double { return Volumes[Idx]; },
		[&](int32 Idx)->int32 { return TransformVertexCounts[Idx]; },
		[&](int32 Idx, int32 VertIdx) -> FVector3d { return GlobalVertices[TransformVertexStarts[Idx] + VertIdx]; },
		ChildrenProximity);

	if (!bEnforceConnectivity)
	{
		// If we're not getting connectivity from the geometry collection, add it based on bounding box overlaps
		ConvexDecomp.InitializeProximityFromDecompositionBoundingBoxOverlaps(1, .5, 1);
	}

	UE::Geometry::FConvexDecomposition3::FMergeSettings MergeSettings;
	MergeSettings.TargetNumParts = SiteCount;

	double TotalVolume = 0;
	for (double Volume : Volumes)
	{
		TotalVolume += Volume;
	}
	MergeSettings.ErrorTolerance = ConcavityTolerance;
	MergeSettings.bErrorToleranceOverridesNumParts = false;
	double MaxVolumeToMerge = TotalVolume / SiteCount; // stop merging parts that are larger than their share of the volume ...
	double MinVolumeToSkip = .2 * TotalVolume / SiteCount; // except still allow merging very small parts into large parts
	MergeSettings.CustomAllowMergeParts =
		[MaxVolumeToMerge, MinVolumeToSkip](const UE::Geometry::FConvexDecomposition3::FConvexPart& PartA, const UE::Geometry::FConvexDecomposition3::FConvexPart& PartB)
		{
			if (PartA.bMustMerge || PartB.bMustMerge)
			{
				return true;
			}
			return ((PartA.SumHullsVolume < MaxVolumeToMerge && PartB.SumHullsVolume < MaxVolumeToMerge) ||
					(PartA.SumHullsVolume < MinVolumeToSkip || PartB.SumHullsVolume < MinVolumeToSkip));
		};
	MergeSettings.MergeCallback = [&Components](int32 ChildA, int32 ChildB)
	{
		Components.Union(ChildA, ChildB);
	};
	
	ConvexDecomp.MergeBest(MergeSettings);
	TArray<int32> CompactIdxToGroupID, GroupIDToCompactIdx;
	int32 PartitionCount = Components.CompactedGroupIndexToGroupID(&CompactIdxToGroupID, &GroupIDToCompactIdx);
	TArray<int32> NewCluster;
	int32 NewClusterIndexStart = GeometryCollection.AddElements(PartitionCount, FGeometryCollection::TransformGroup);
	bool bHasEmptyClusters = false;
	for (int32 Index = 0; Index < PartitionCount; ++Index)
	{
		NewCluster.Reset();
		int32 GroupID = CompactIdxToGroupID[Index];
		for (int32 ChildIdx = 0; ChildIdx < Children.Num(); ++ChildIdx)
		{
			if (Components.Find(ChildIdx) == GroupID)
			{
				NewCluster.Add(Children[ChildIdx]);
			}
		}
		if (bAvoidIsolated && NewCluster.Num() == 1)
		{
			bHasEmptyClusters = true;
			NewCluster.Reset();
		}

		int32 NewClusterIndex = NewClusterIndexStart + Index;
		GeometryCollection.Parent[NewClusterIndex] = ClusterIndex;
		GeometryCollection.Children[ClusterIndex].Add(NewClusterIndex);
		GeometryCollection.BoneName[NewClusterIndex] = "ClusterBone";
		GeometryCollection.Children[NewClusterIndex] = TSet<int32>(NewCluster);
		GeometryCollection.SimulationType[NewClusterIndex] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		GeometryCollection.Transform[NewClusterIndex] = FTransform3f::Identity;
		GeometryCollectionAlgo::ParentTransforms(&GeometryCollection, NewClusterIndex, NewCluster);
	}
	if (bHasEmptyClusters)
	{
		FGeometryCollectionClusteringUtility::RemoveDanglingClusters(&GeometryCollection);
	}
	FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(&GeometryCollection, ClusterIndex);
	FGeometryCollectionClusteringUtility::RecursivelyUpdateChildBoneNames(ClusterIndex, GeometryCollection.Children, GeometryCollection.BoneName);
	FGeometryCollectionClusteringUtility::ValidateResults(&GeometryCollection);
}

void FFractureEngineClustering::AutoCluster(FGeometryCollection& GeometryCollection,
	const int32 ClusterIndex,
	const EFractureEngineClusterSizeMethod ClusterSizeMethod,
	const uint32 SiteCount,
	const float SiteCountFraction,
	const float SiteSize,
	const bool bEnforceConnectivity,
	const bool bAvoidIsolated,
	const bool bEnforceSiteParameters,
	const int32 InGridX,
	const int32 InGridY,
	const int32 InGridZ,
	const float MinimumClusterSize,
	const int32 KMeansIterations,
	const bool bPreferConvexity,
	const float ConcavityTolerance)
{
	// Used by the ByGrid method, which manually distributes initial positions
	TArray<FVector> PartitionPositions;

	int32 DesiredSiteCountToUse = 1;
	int32 IterationsToUse = KMeansIterations;
	int32 NumChildren = GeometryCollection.Children[ClusterIndex].Num();
	if (ClusterSizeMethod == EFractureEngineClusterSizeMethod::ByNumber)
	{
		DesiredSiteCountToUse = SiteCount;
	}
	else if (ClusterSizeMethod == EFractureEngineClusterSizeMethod::ByFractionOfInput)
	{
		DesiredSiteCountToUse = FMath::Max(2, NumChildren * SiteCountFraction);
	}
	else if (ClusterSizeMethod == EFractureEngineClusterSizeMethod::BySize)
	{
		float TotalVolume = GeometryCollection.GetBoundingBox().GetBox().GetVolume();
		float DesiredVolume = FMath::Pow(SiteSize, 3);
		DesiredSiteCountToUse = FMath::Max(1, TotalVolume / DesiredVolume);
	}
	else if (ClusterSizeMethod == EFractureEngineClusterSizeMethod::ByGrid)
	{
		PartitionPositions = GenerateGridSites(GeometryCollection, ClusterIndex, InGridX, InGridY, InGridZ);
		DesiredSiteCountToUse = PartitionPositions.Num();
	}

	if (bPreferConvexity)
	{
		ConvexityBasedCluster(GeometryCollection, ClusterIndex, SiteCount, bEnforceConnectivity, bAvoidIsolated, ConcavityTolerance);
		return;
	}

	FVoronoiPartitioner VoronoiPartition(&GeometryCollection, ClusterIndex);

	// Stop if we only want one cluster or there aren't enough children to do any clustering
	if (DesiredSiteCountToUse <= 1 || NumChildren <= 1)
	{
		return;
	}

	bool bNeedsVolume = MinimumClusterSize > 0;
	if (bNeedsVolume)
	{
		FGeometryCollectionConvexUtility::SetVolumeAttributes(&GeometryCollection);
	}

	int32 SiteCountToUse = DesiredSiteCountToUse;
	int32 PreviousPartitionCount = TNumericLimits<int32>::Max();
	bool bIterate = false;
	do
	{
		VoronoiPartition.KMeansPartition(SiteCountToUse, IterationsToUse, PartitionPositions);
		if (VoronoiPartition.GetPartitionCount() == 0)
		{
			return;
		}

		bool bNeedProximity = bEnforceConnectivity || bAvoidIsolated;
		if (bNeedProximity)
		{
			FGeometryCollectionProximityUtility ProximityUtility(&GeometryCollection);
			ProximityUtility.UpdateProximity();
		}

		if (bEnforceConnectivity)
		{
			VoronoiPartition.SplitDisconnectedPartitions(&GeometryCollection);
		}

		// Note: Previously if bAvoidIsolated was set, we'd attempt to merge clusters of one here via
		//  VoronoiPartition.MergeSingleElementPartitions(&GeometryCollection);
		// However this results in some overly-large clusters, especially with grid clustering,
		// so we prefer to simply skip clustering in such cases.

		if (MinimumClusterSize > 0)
		{
			// attempt to remove isolated via cluster merging (may not succeed, as it only merges if there is a cluster in proximity)
			VoronoiPartition.MergeSmallPartitions(&GeometryCollection, MinimumClusterSize);
		}
		const int32 IsolatedPartitionToIgnore = bAvoidIsolated ? VoronoiPartition.GetIsolatedPartitionCount() : 0;
		const int32 PartitionCount = VoronoiPartition.GetNonEmptyPartitionCount() - IsolatedPartitionToIgnore;
		bIterate = bEnforceSiteParameters				 // Is the feature enabled?
			&& PartitionPositions.IsEmpty()				 // Is the explicit position array empty? (otherwise, site count is ignored)
			&& (PartitionCount > DesiredSiteCountToUse)  // Have we reached the desired outcome
			&& (SiteCountToUse > 1)						 // Is SiteCount large enough ?
			&& (PartitionCount < PreviousPartitionCount);// Are we progressing ?
		if(bIterate)
		{
			SiteCountToUse--;
		}
	} while (bIterate);

	int32 NonEmptyPartitionCount = VoronoiPartition.GetNonEmptyPartitionCount();
	bool bHasEmptyClusters = NonEmptyPartitionCount > 0;

	if (bAvoidIsolated && NonEmptyPartitionCount == 1)
	{
		return;
	}

	int32 PartitionCount = VoronoiPartition.GetPartitionCount();
	int32 NewClusterIndexStart = GeometryCollection.AddElements(PartitionCount, FGeometryCollection::TransformGroup);

	for (int32 Index = 0; Index < PartitionCount; ++Index)
	{
		TArray<int32> NewCluster = VoronoiPartition.GetPartition(Index);
		if (bAvoidIsolated && NewCluster.Num() == 1)
		{
			bHasEmptyClusters = true;
			NewCluster.Reset();
		}

		int32 NewClusterIndex = NewClusterIndexStart + Index;
		GeometryCollection.Parent[NewClusterIndex] = ClusterIndex;
		GeometryCollection.Children[ClusterIndex].Add(NewClusterIndex);
		GeometryCollection.BoneName[NewClusterIndex] = "ClusterBone";
		GeometryCollection.Children[NewClusterIndex] = TSet<int32>(NewCluster);
		GeometryCollection.SimulationType[NewClusterIndex] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		GeometryCollection.Transform[NewClusterIndex] = FTransform3f::Identity;
		GeometryCollectionAlgo::ParentTransforms(&GeometryCollection, NewClusterIndex, NewCluster);
	}
	if (bHasEmptyClusters)
	{
		FGeometryCollectionClusteringUtility::RemoveDanglingClusters(&GeometryCollection);
	}
	FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(&GeometryCollection, ClusterIndex);
	FGeometryCollectionClusteringUtility::RecursivelyUpdateChildBoneNames(ClusterIndex, GeometryCollection.Children, GeometryCollection.BoneName);
	FGeometryCollectionClusteringUtility::ValidateResults(&GeometryCollection);
}

TArray<FVector> FFractureEngineClustering::GenerateGridSites(
	const FGeometryCollection& GeometryCollection,
	const TArray<int32>& BoneIndices,
	const int32 GridX,
	const int32 GridY,
	const int32 GridZ)
{
	TArray<FVector> AllPoints;
	FFractureEngineBoneSelection Selection(GeometryCollection, BoneIndices);

	Selection.ConvertSelectionToClusterNodes();

	for (const int32 ClusterIndex : Selection.GetSelectedBones())
	{
		AllPoints.Append(GenerateGridSites(GeometryCollection, ClusterIndex, GridX, GridY, GridZ));
	}
	return AllPoints;
}

TArray<FVector> FFractureEngineClustering::GenerateGridSites(
	const FGeometryCollection& GeometryCollection,
	const int32 ClusterIndex,
	const int32 InGridX,
	const int32 InGridY,
	const int32 InGridZ,
	FBox* OutBounds)
{
	TArray<FVector> PartitionPositions;

	FBox Bounds = FVoronoiPartitioner::GenerateBounds(&GeometryCollection, ClusterIndex);
	if (OutBounds)
	{
		*OutBounds = Bounds;
	}
	int32 GridX = FMath::Max(1, InGridX);
	int32 GridY = FMath::Max(1, InGridY);
	int32 GridZ = FMath::Max(1, InGridZ);
	PartitionPositions.Reserve(GridX * GridY * GridZ);

	auto StepToParam = [](int32 Step, int32 NumSteps) -> double
	{
		return (double(Step) + .5) / double(NumSteps);
	};

	for (int32 StepX = 0; StepX < GridX; ++StepX)
	{
		double Xt = StepToParam(StepX, GridX);
		double X = FMath::Lerp(Bounds.Min.X, Bounds.Max.X, Xt);
		for (int32 StepY = 0; StepY < GridY; ++StepY)
		{
			double Yt = StepToParam(StepY, GridY);
			double Y = FMath::Lerp(Bounds.Min.Y, Bounds.Max.Y, Yt);
			for (int32 StepZ = 0; StepZ < GridZ; ++StepZ)
			{
				double Zt = StepToParam(StepZ, GridZ);
				double Z = FMath::Lerp(Bounds.Min.Z, Bounds.Max.Z, Zt);
				PartitionPositions.Emplace(X, Y, Z);
			}
		}
	}

	return PartitionPositions;
}


bool FFractureEngineClustering::ClusterSelected(
	FGeometryCollection& GeometryCollection,
	TArray<int32>& Selection)
{
	if (GeometryCollection.NumElements(FGeometryCollection::TransformGroup) == 0)
	{
		return false;
	}

	GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(GeometryCollection);
	SelectionFacade.RemoveRootNodes(Selection);
	SelectionFacade.Sanitize(Selection);

	if (Selection.Num() <= 1)
	{
		// Don't make a cluster of 1
		return false;
	}

	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(GeometryCollection);
	int32 StartTransformCount = GeometryCollection.NumElements(FGeometryCollection::TransformGroup);
	int32 LowestCommonAncestor = FGeometryCollectionClusteringUtility::FindLowestCommonAncestor(&GeometryCollection, Selection);
	// Cluster selected bones beneath common parent
	if (LowestCommonAncestor != INDEX_NONE)
	{
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNodeWithParent(&GeometryCollection, LowestCommonAncestor, Selection, true);
		GeometryCollection::GenerateTemporaryGuids(&GeometryCollection, StartTransformCount, true);
		FGeometryCollectionClusteringUtility::RemoveDanglingClusters(&GeometryCollection);
		HierarchyFacade.GenerateLevelAttribute();

		return true;
	}

	return false;
}

bool FFractureEngineClustering::MergeSelectedClusters(FGeometryCollection& GeometryCollection, TArray<int32>& Selection)
{
	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(GeometryCollection);

	// Merging a selection element does not make sense, skip it 
	if (Selection.Num() <= 1)
	{
		return false;
	}

	const int32 NumTransforms = GeometryCollection.NumElements(FGeometryCollection::TransformGroup);
	if (Selection.Num() > NumTransforms)
	{
		return false;
	}

	GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(GeometryCollection);
	SelectionFacade.ConvertEmbeddedSelectionToParents(Selection); // embedded geo must stay attached to parent

	// Collect children of context clusters
	TArray<int32> ChildBones;
	int32 StartTransformCount = GeometryCollection.NumElements(FGeometryCollection::TransformGroup);
	ChildBones.Reserve(StartTransformCount);
	bool bHasClusters = false;
	for (int32 Select : Selection)
	{
		if (GeometryCollection.SimulationType[Select] == FGeometryCollection::ESimulationTypes::FST_Clustered)
		{
			bHasClusters = true;
			ChildBones.Append(HierarchyFacade.GetChildrenAsArray(Select));
		}
		else
		{
			ChildBones.Add(Select);
		}
	}

	// If there were no clusters in the selection, we create a cluster
	if (!bHasClusters)
	{
		// Cluster selected bones beneath common parent
		const int32 LowestCommonAncestor = FGeometryCollectionClusteringUtility::FindLowestCommonAncestor(&GeometryCollection, Selection);
		if (LowestCommonAncestor != INDEX_NONE && HierarchyFacade.GetParent(LowestCommonAncestor) != INDEX_NONE)
		{
			FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNodeWithParent(&GeometryCollection, LowestCommonAncestor, Selection, true);
			GeometryCollection::GenerateTemporaryGuids(&GeometryCollection, StartTransformCount, true);
			Selection.SetNum(1);
			Selection[0] = LowestCommonAncestor;
			if (FGeometryCollectionClusteringUtility::RemoveDanglingClusters(&GeometryCollection))
			{
				// Bone index is unreliable after removal of some bones
				Selection.Empty();
			}
			HierarchyFacade.GenerateLevelAttribute();
			return true;
		}
	}
	else
	{
		int32 MergeNode = FGeometryCollectionClusteringUtility::PickBestNodeToMergeTo(&GeometryCollection, Selection);
		if (MergeNode >= 0)
		{
			FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingNode(&GeometryCollection, MergeNode, ChildBones);
			Selection.SetNum(1);
			Selection[0] = MergeNode;
			if (FGeometryCollectionClusteringUtility::RemoveDanglingClusters(&GeometryCollection))
			{
				// Bone index is unreliable after removal of some bones
				Selection.Empty();
			}
			HierarchyFacade.GenerateLevelAttribute();
			return true;
		}
	}

	return false;
}


bool FFractureEngineClustering::ClusterMagnet(
	FGeometryCollection& GeometryCollection,
	TArray<int32>& InOutSelection,
	int32 Iterations
)
{
	using namespace UE::Private::ClusterMagnet;

	if (GeometryCollection.NumElements(FGeometryCollection::TransformGroup) == 0)
	{
		return false;
	}

	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(GeometryCollection);
	HierarchyFacade.GenerateLevelAttribute();
	const TManagedArray<int32>& Levels = GeometryCollection.GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

	GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(GeometryCollection);
	SelectionFacade.Sanitize(InOutSelection);
	SelectionFacade.ConvertEmbeddedSelectionToParents(InOutSelection); // embedded geo must stay attached to parent

	const TManagedArray<TSet<int32>>& Children = GeometryCollection.Children;
	TMap<int32, TArray<int32>> ClusteredSelection = SelectionFacade.GetClusteredSelections(InOutSelection);
	
	for (TPair<int32, TArray<int32>>& Group : ClusteredSelection)
	{
		if (Group.Key == INDEX_NONE) // Group is top level
		{
			continue;
		}

		// We have the connections for the leaf nodes of our geometry collection. We want to percolate those up to the top nodes.
		TMap<int32, TSet<int32>> TopNodeConnectivity = InitializeConnectivity(Children[Group.Key], &GeometryCollection, Levels[Group.Key]+1);

		// Separate the top nodes into cluster magnets and a pool of available nodes.
		TArray<FClusterMagnet> ClusterMagnets;
		TSet<int32> RemainingPool;
		SeparateClusterMagnets(Children[Group.Key], Group.Value, TopNodeConnectivity, ClusterMagnets, RemainingPool);

		for (int32 Iteration = 0; Iteration < Iterations; ++Iteration)
		{
			bool bNeighborsAbsorbed = false;

			// each cluster gathers adjacent nodes from the pool
			for (FClusterMagnet& ClusterMagnet : ClusterMagnets)
			{
				bNeighborsAbsorbed |= AbsorbClusterNeighbors(TopNodeConnectivity, ClusterMagnet, RemainingPool);
			}

			// early termination
			if (!bNeighborsAbsorbed)
			{
				break;
			}
		}

		// Create new clusters from the cluster magnets
		for (const FClusterMagnet& ClusterMagnet : ClusterMagnets)
		{
			if (ClusterMagnet.ClusteredNodes.Num() > 1)
			{
				TArray<int32> NewChildren = ClusterMagnet.ClusteredNodes.Array();
				NewChildren.Sort();
				FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(&GeometryCollection, NewChildren[0], NewChildren, false, false);
			}
		}
	}

	return true;
}

