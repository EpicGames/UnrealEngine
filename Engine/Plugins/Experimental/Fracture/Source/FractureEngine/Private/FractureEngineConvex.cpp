// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineConvex.h"
#include "Chaos/Convex.h"
#include "CompGeom/ConvexDecomposition3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "ProjectionTargets.h"
#include "MeshSimplification.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "DisjointSet.h"
#include "VectorUtil.h"
#include "Util/IndexPriorityQueue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureEngineConvex)

namespace
{
	// Local helpers for converting convex hulls to dynamic meshes, used to run geometry processing tasks on convex hulls (e.g., simplification, computing negative space)

	static void AppendConvexHullToCompactDynamicMesh(const ::Chaos::FConvex* InConvexHull, UE::Geometry::FDynamicMesh3& Mesh, FTransform* OptionalTransform = nullptr, 
		bool bFixNonmanifoldWithDuplicates = false, bool bInvertFaces = false, bool bRecompute = false)
	{
		check(Mesh.IsCompact());

		const int32 NumV = InConvexHull->NumVertices();
		const int32 NumP = InConvexHull->NumPlanes();
		int32 StartV = Mesh.MaxVertexID();
		for (int32 VIdx = 0; VIdx < NumV; ++VIdx)
		{
			FVector3d V = (FVector3d)InConvexHull->GetVertex(VIdx);
			if (OptionalTransform)
			{
				V = OptionalTransform->TransformPosition(V);
			}
			int32 MeshVIdx = Mesh.AppendVertex(V);
			checkSlow(MeshVIdx == VIdx + StartV); // Must be true because the mesh is compact
		}

		// Recompute the convex hull from scratch
		if (bRecompute)
		{
			UE::Geometry::FConvexHull3d Hull;
			Hull.Solve(NumV, [InConvexHull](int32 VID) { return (FVector)InConvexHull->GetVertex(VID); });
			const TArray<UE::Geometry::FIndex3i>& Tris = Hull.GetTriangles();
			for (int32 TriIdx = 0; TriIdx < Tris.Num(); ++TriIdx)
			{
				UE::Geometry::FIndex3i OffsetTri = Tris[TriIdx].GetOffsetBy(StartV);
				if (!bInvertFaces) // recomputed hull has faces w/ opposite winding from chaos
				{
					Swap(OffsetTri.B, OffsetTri.C);
				}
				Mesh.AppendTriangle(OffsetTri);
			}
			return;
		}

		// Not recomputing -- copy the chaos mesh structure out
		const ::Chaos::FConvexStructureData& ConvexStructure = InConvexHull->GetStructureData();
		for (int32 PIdx = 0; PIdx < NumP; ++PIdx)
		{
			const int32 NumFaceV = ConvexStructure.NumPlaneVertices(PIdx);
			const int32 V0 = StartV + ConvexStructure.GetPlaneVertex(PIdx, 0);
			for (int32 SubIdx = 1; SubIdx + 1 < NumFaceV; ++SubIdx)
			{
				int32 V1 = StartV + ConvexStructure.GetPlaneVertex(PIdx, SubIdx);
				int32 V2 = StartV + ConvexStructure.GetPlaneVertex(PIdx, SubIdx + 1);
				if (bInvertFaces)
				{
					Swap(V1, V2);
				}
				int32 ResultTID = Mesh.AppendTriangle(UE::Geometry::FIndex3i(V0, V1, V2));
				if (bFixNonmanifoldWithDuplicates && ResultTID == UE::Geometry::FDynamicMesh3::NonManifoldID)
				{
					// failed to append due to a non-manifold triangle; try adding all the vertices independently so we at least capture the shape
					// note: this should not happen for normal convex hulls, but the current convex hull algorithm does some aggressive face merging that sometimes creates weird geometry
					UE::Geometry::FIndex3i DuplicateVerts(
						Mesh.AppendVertex(Mesh.GetVertex(V0)),
						Mesh.AppendVertex(Mesh.GetVertex(V1)),
						Mesh.AppendVertex(Mesh.GetVertex(V2))
					);
					Mesh.AppendTriangle(DuplicateVerts);
				}
			}
		}
	}

	static UE::Geometry::FDynamicMesh3 ConvexHullToDynamicMesh(const ::Chaos::FConvex* InConvexHull)
	{
		UE::Geometry::FDynamicMesh3 Mesh;
		AppendConvexHullToCompactDynamicMesh(InConvexHull, Mesh);
		return Mesh;
	}
}

namespace UE::FractureEngine::Convex
{
	bool GetConvexHullsAsDynamicMesh(const FManagedArrayCollection& Collection, UE::Geometry::FDynamicMesh3& OutMesh, bool bRestrictToSelection, const TArrayView<const int32> TransformSelection, bool bRecomputeHulls)
	{
		OutMesh.Clear();

		if (!FGeometryCollectionConvexUtility::HasConvexHullData(&Collection))
		{
			// nothing to append
			return false;
		}

		const TManagedArray<TSet<int32>>& TransformToConvexInds = Collection.GetAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		const TManagedArray<Chaos::FConvexPtr>& ConvexHulls = Collection.GetAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);
		TArray<FTransform> GlobalTransformArray = TransformFacade.ComputeCollectionSpaceTransforms();

		auto AppendBone = [&TransformToConvexInds, &ConvexHulls, &GlobalTransformArray, &OutMesh, bRecomputeHulls](int32 BoneIdx) -> bool
		{
			if (BoneIdx < 0 || BoneIdx >= TransformToConvexInds.Num())
			{
				// invalid bone index
				return false;
			}
			for (int32 ConvexIdx : TransformToConvexInds[BoneIdx])
			{
				constexpr bool bConvertNonManifold = true; // Add non-manifold faces so they are still included in the debug visualization
				constexpr bool bInvertFaces = true; // FConvex mesh data appears to have opposite default winding from what we expect for triangle meshes
				AppendConvexHullToCompactDynamicMesh(ConvexHulls[ConvexIdx].GetReference(), OutMesh, &GlobalTransformArray[BoneIdx], bConvertNonManifold, bInvertFaces, bRecomputeHulls);
			}
			return true;
		};

		bool bNoFailures = true;

		if (bRestrictToSelection)
		{
			for (int32 BoneIdx : TransformSelection)
			{
				bool bSuccess = AppendBone(BoneIdx);
				bNoFailures = bNoFailures && bSuccess;
			}
		}
		else
		{
			for (int32 BoneIdx = 0; BoneIdx < Collection.NumElements(FGeometryCollection::TransformGroup); ++BoneIdx)
			{
				bool bSuccess = AppendBone(BoneIdx);
				bNoFailures = bNoFailures && bSuccess;
			}
		}

		return bNoFailures;
	}

	bool GetConvexHullsAsDynamicMeshes(const FManagedArrayCollection& Collection, TArray<UE::Geometry::FDynamicMesh3>& OutMeshes, bool bRestrictToSelection, const TArrayView<const int32> TransformSelection, bool bRecomputeHulls)
	{
		OutMeshes.Reset();

		if (!FGeometryCollectionConvexUtility::HasConvexHullData(&Collection))
		{
			// nothing to append
			return false;
		}

		const TManagedArray<TSet<int32>>& TransformToConvexInds = Collection.GetAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		const TManagedArray<Chaos::FConvexPtr>& ConvexHulls = Collection.GetAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);
		TArray<FTransform> GlobalTransformArray = TransformFacade.ComputeCollectionSpaceTransforms();

		auto AppendBone = [&TransformToConvexInds, &ConvexHulls, &GlobalTransformArray, &OutMeshes, bRecomputeHulls](int32 BoneIdx) -> bool
			{
				if (BoneIdx < 0 || BoneIdx >= TransformToConvexInds.Num())
				{
					// invalid bone index
					return false;
				}
				for (int32 ConvexIdx : TransformToConvexInds[BoneIdx])
				{
					constexpr bool bConvertNonManifold = true; // Add non-manifold faces so they are still included in the debug visualization
					constexpr bool bInvertFaces = true; // FConvex mesh data appears to have opposite default winding from what we expect for triangle meshes
					UE::Geometry::FDynamicMesh3& DynamicMesh = OutMeshes.AddDefaulted_GetRef();
					AppendConvexHullToCompactDynamicMesh(ConvexHulls[ConvexIdx].GetReference(), DynamicMesh, &GlobalTransformArray[BoneIdx], bConvertNonManifold, bInvertFaces, bRecomputeHulls);
				}
				return true;
			};

		bool bNoFailures = true;

		if (bRestrictToSelection)
		{
			for (int32 BoneIdx : TransformSelection)
			{
				bool bSuccess = AppendBone(BoneIdx);
				bNoFailures = bNoFailures && bSuccess;
			}
		}
		else
		{
			for (int32 BoneIdx = 0; BoneIdx < Collection.NumElements(FGeometryCollection::TransformGroup); ++BoneIdx)
			{
				bool bSuccess = AppendBone(BoneIdx);
				bNoFailures = bNoFailures && bSuccess;
			}
		}

		return bNoFailures;
	}


	bool SimplifyConvexHulls(FManagedArrayCollection& Collection, const FSimplifyHullSettings& Settings, bool bRestrictToSelection, const TArrayView<const int32> TransformSelection)
	{
		if (!FGeometryCollectionConvexUtility::HasConvexHullData(&Collection))
		{
			// nothing to simplify
			return false;
		}
		
		TManagedArray<TSet<int32>>& TransformToConvexInds = Collection.ModifyAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		TManagedArray<Chaos::FConvexPtr>& ConvexHulls = Collection.ModifyAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
		
		auto SimplifyBone = [&TransformToConvexInds, &ConvexHulls, &Settings](int32 BoneIdx) -> bool
		{
			if (BoneIdx < 0 || BoneIdx >= TransformToConvexInds.Num())
			{
				// invalid bone index
				return false;
			}
			bool bNoFailures = true;
			for (int32 ConvexIdx : TransformToConvexInds[BoneIdx])
			{
				// Note: We construct a new convex to hold the simplified result, rather than directly overwriting the old result
				// to make sure that cached collections holding the same FConvexPtr aren't accidentally updated w/ the simplfied version
				Chaos::FConvexPtr SimplifiedConvex(new Chaos::FConvex);
				bool bSuccess = SimplifyConvexHull(ConvexHulls[ConvexIdx].GetReference(), SimplifiedConvex.GetReference(), Settings);
				ConvexHulls[ConvexIdx] = SimplifiedConvex;
				bNoFailures = bNoFailures && bSuccess;
			}
			return bNoFailures;
		};

		bool bNoFailures = true;

		if (bRestrictToSelection)
		{
			for (int32 BoneIdx : TransformSelection)
			{
				bool bSuccess = SimplifyBone(BoneIdx);
				bNoFailures = bNoFailures && bSuccess;
			}
		}
		else
		{
			for (int32 BoneIdx = 0; BoneIdx < Collection.NumElements(FGeometryCollection::TransformGroup); ++BoneIdx)
			{
				bool bSuccess = SimplifyBone(BoneIdx);
				bNoFailures = bNoFailures && bSuccess;
			}
		}

		return bNoFailures;
	}

	bool SimplifyConvexHull(const ::Chaos::FConvex* InConvexHull, ::Chaos::FConvex* OutConvexHull, const FSimplifyHullSettings& Settings)
	{
		if (!InConvexHull || !OutConvexHull || !InConvexHull->HasStructureData())
		{
			return false;
		}

		const ::Chaos::FConvexStructureData& ConvexStructure = InConvexHull->GetStructureData();
		const int32 NumP = InConvexHull->NumPlanes();

		// Check if no simplification required, and skip simplification in that case
		int32 ExpectNumT = 0;
		for (int32 PIdx = 0; PIdx < NumP; ++PIdx)
		{
			const int32 NumFaceV = ConvexStructure.NumPlaneVertices(PIdx);
			ExpectNumT += FMath::Max(0, NumFaceV - 2);
		}
		if (Settings.bUseTargetTriangleCount && ExpectNumT <= Settings.TargetTriangleCount)
		{
			if (OutConvexHull != InConvexHull)
			{
				*OutConvexHull = MoveTemp(*InConvexHull->RawCopyAsConvex());
			}
			return true;
		}

		if (Settings.SimplifyMethod == EConvexHullSimplifyMethod::MeshQSlim)
		{
			// Convert to DynamicMesh to run simplifier
			UE::Geometry::FDynamicMesh3 Mesh = ConvexHullToDynamicMesh(InConvexHull);

			// Run simplification
			UE::Geometry::FVolPresMeshSimplification Simplifier(&Mesh);

			Simplifier.CollapseMode =
				Settings.bUseExistingVertexPositions ?
					UE::Geometry::FVolPresMeshSimplification::ESimplificationCollapseModes::MinimalExistingVertexError
				: 	UE::Geometry::FVolPresMeshSimplification::ESimplificationCollapseModes::MinimalQuadricPositionError;
			if (Settings.bUseGeometricTolerance)
			{
				Simplifier.GeometricErrorConstraint = UE::Geometry::FVolPresMeshSimplification::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
				Simplifier.GeometricErrorTolerance = Settings.ErrorTolerance;
			}
			if (Settings.bUseGeometricTolerance)
			{
				// Simplify to the smallest non-degenerate number of triangles, relying on geometric error criteria
				UE::Geometry::FDynamicMesh3 ProjectionTargetMesh(Mesh);
				UE::Geometry::FDynamicMeshAABBTree3 ProjectionTargetSpatial(&ProjectionTargetMesh, true);
				UE::Geometry::FMeshProjectionTarget ProjTarget(&ProjectionTargetMesh, &ProjectionTargetSpatial);
				Simplifier.SetProjectionTarget(&ProjTarget);
				int32 TargetTriCount = Settings.bUseTargetTriangleCount ? Settings.TargetTriangleCount : 4;
				Simplifier.SimplifyToTriangleCount(TargetTriCount);
			}
			else if (Settings.bUseTargetTriangleCount)
			{
				Simplifier.SimplifyToTriangleCount(Settings.TargetTriangleCount);
			}
			else
			{
				// Note: Quadric error threshold doesn't have the same geometric meaning as distance; this is not equivalent to using a geometric error tolerance
				Simplifier.SimplifyToMaxError(Settings.ErrorTolerance * Settings.ErrorTolerance);
			}

			TArray<::Chaos::FVec3f> NewConvexVerts;
			NewConvexVerts.Reserve(Mesh.VertexCount());
			for (int32 VIdx : Mesh.VertexIndicesItr())
			{
				NewConvexVerts.Add((::Chaos::FVec3f)Mesh.GetVertex(VIdx));
			}
			*OutConvexHull = ::Chaos::FConvex(NewConvexVerts, InConvexHull->GetMargin(), ::Chaos::FConvexBuilder::EBuildMethod::Default);
		}
		else if (Settings.SimplifyMethod == EConvexHullSimplifyMethod::AngleTolerance)
		{
			FDisjointSet PlaneGroups(NumP);
			TArray<FVector> PlaneNormals; PlaneNormals.SetNumUninitialized(NumP);
			TArray<double> PlaneAreas; PlaneAreas.SetNumUninitialized(NumP);
			for (int32 PlaneIdx = 0; PlaneIdx < NumP; ++PlaneIdx)
			{
				Chaos::FVec3 Pos, Normal;
				InConvexHull->GetPlaneNX(PlaneIdx, Normal, Pos);
				PlaneNormals[PlaneIdx] = (FVector)Normal;
				Chaos::FVec3 V0 = InConvexHull->GetVertex(InConvexHull->GetPlaneVertex(PlaneIdx, 0));
				double AreaSum = 0;
				for (int32 SubIdx = 1; SubIdx + 1 < InConvexHull->NumPlaneVertices(PlaneIdx); ++SubIdx)
				{
					Chaos::FVec3 V1 = InConvexHull->GetVertex(InConvexHull->GetPlaneVertex(PlaneIdx, SubIdx));
					Chaos::FVec3 V2 = InConvexHull->GetVertex(InConvexHull->GetPlaneVertex(PlaneIdx, SubIdx+1));
					AreaSum += UE::Geometry::VectorUtil::Area(V0, V1, V2);
				}
				PlaneAreas[PlaneIdx] = AreaSum;
			}

			int32 NumEdges = InConvexHull->NumEdges();
			UE::Geometry::FIndexPriorityQueue EdgeQueue(NumEdges);

			const double AreaThreshold = FMath::Max(UE_DOUBLE_KINDA_SMALL_NUMBER, Settings.SmallAreaThreshold);
			auto GetMergeWeight = [&InConvexHull, &PlaneGroups, &PlaneAreas, &PlaneNormals, AreaThreshold](int32 EdgeIdx, bool bHasGroups) -> float
			{
				int32 P[2]{ InConvexHull->GetEdgePlane(EdgeIdx, 0), InConvexHull->GetEdgePlane(EdgeIdx, 1) };
				if (bHasGroups)
				{
					P[0] = PlaneGroups.Find(P[0]);
					P[1] = PlaneGroups.Find(P[1]);
					if (P[0] == P[1])
					{
						// return a value higher than the max possible angle threshold if the plane groups are already merged
						constexpr float CannotMergeValue = 4;
						return CannotMergeValue;
					}
				}
				// Planes with small area can be merged into any neighbor (e.g., to filter degenerate or less meaningful normals)
				if (PlaneAreas[P[0]] < AreaThreshold || PlaneAreas[P[1]] < AreaThreshold)
				{
					return 0;
				}
				float NormalAlignment = 1 - float(PlaneNormals[P[0]].Dot(PlaneNormals[P[1]]));
				return NormalAlignment;
			};
			
			const float Threshold = 1 - FMath::Cos(FMath::DegreesToRadians(Settings.AngleThreshold));
			for (int32 EdgeIdx = 0; EdgeIdx < NumEdges; ++EdgeIdx)
			{
				float Wt = GetMergeWeight(EdgeIdx, false);
				if (Wt < Threshold)
				{
					EdgeQueue.Insert(EdgeIdx, Wt);
				}
			}

			int32 NumRemainingPlanes = NumP;
			while (EdgeQueue.GetCount() > 0)
			{
				int32 EdgeIdx = EdgeQueue.Dequeue();
				float Wt = GetMergeWeight(EdgeIdx, true);
				if (Wt < Threshold)
				{
					int32 P[2]{ PlaneGroups.Find(InConvexHull->GetEdgePlane(EdgeIdx, 0)), PlaneGroups.Find(InConvexHull->GetEdgePlane(EdgeIdx, 1)) };
					PlaneGroups.Union(P[0], P[1]);
					int32 Parent = PlaneGroups.Find(P[0]);
					FVector Normal = PlaneNormals[P[0]] * PlaneAreas[P[0]] + PlaneNormals[P[1]] * PlaneAreas[P[1]];
					Normal.Normalize();
					double AreaSum = PlaneAreas[P[0]] + PlaneAreas[P[1]];
					PlaneNormals[Parent] = Normal;
					PlaneAreas[Parent] = AreaSum;
					NumRemainingPlanes--;
				}
				if (NumRemainingPlanes < Settings.TargetTriangleCount)
				{
					break;
				}
			}

			TArray<::Chaos::FVec3f> NewConvexVerts;
			int32 NumV = InConvexHull->NumVertices();
			NewConvexVerts.Reserve(NumV);

			// Note: We should be able to do this more directly by using ConvexStructure.FindVertexPlanes,
			// but currently this appears to sometimes not find all the vertex planes, so over-simplifies.
			TArray<FIntVector> VertPlaneGroups;
			VertPlaneGroups.Init(FIntVector(-1, -1, -1), NumV);
			for (int32 PIdx = 0; PIdx < NumP; ++PIdx)
			{
				int32 Group = PlaneGroups.Find(PIdx);
				int32 NumFaceVert = InConvexHull->NumPlaneVertices(PIdx);
				for (int32 SubIdx = 0; SubIdx < NumFaceVert; ++SubIdx)
				{
					int32 VIdx = InConvexHull->GetPlaneVertex(PIdx, SubIdx);
					for (int32 Idx = 0; Idx < 3; ++Idx)
					{
						if (VertPlaneGroups[VIdx][Idx] == -1)
						{
							VertPlaneGroups[VIdx][Idx] = Group;
							break;
						}
						else if (VertPlaneGroups[VIdx][Idx] == Group)
						{
							break;
						}
					}
				}
			}
			for (int32 VIdx = 0; VIdx < NumV; ++VIdx)
			{
				if (VertPlaneGroups[VIdx][2] > -1)
				{
					NewConvexVerts.Add(InConvexHull->GetVertex(VIdx));
				}
			}

			*OutConvexHull = ::Chaos::FConvex(NewConvexVerts, InConvexHull->GetMargin(), ::Chaos::FConvexBuilder::EBuildMethod::Default);
		}

		return true;
	}

	bool ComputeConvexHullsNegativeSpace(FManagedArrayCollection& Collection, UE::Geometry::FSphereCovering& OutNegativeSpace, const UE::Geometry::FNegativeSpaceSampleSettings& Settings, bool bRestrictToSelection, const TArrayView<const int32> TransformSelection, bool bFromRigidTransforms)
	{
		if (!FGeometryCollectionConvexUtility::HasConvexHullData(&Collection))
		{
			return false;
		}

		TManagedArray<TSet<int32>>& TransformToConvexInds = Collection.ModifyAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		TManagedArray<Chaos::FConvexPtr>& ConvexHulls = Collection.ModifyAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);

		UE::Geometry::FDynamicMesh3 CombinedMesh;

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);
		GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(Collection);
		TArray<int> UseSelection;
		if (!bRestrictToSelection)
		{
			UseSelection = bFromRigidTransforms ? SelectionFacade.SelectLeaf() : SelectionFacade.SelectAll();
		}
		else
		{
			UseSelection.Append(TransformSelection);
			if (bFromRigidTransforms)
			{
				SelectionFacade.ConvertSelectionToRigidNodes(UseSelection);
			}
		}
		
		TArray<FTransform> GlobalTransformArray = TransformFacade.ComputeCollectionSpaceTransforms();
		
		auto ProcessBone = [&TransformToConvexInds, &ConvexHulls, &CombinedMesh, &GlobalTransformArray](int32 BoneIdx) -> bool
		{
			if (BoneIdx < 0 || BoneIdx >= TransformToConvexInds.Num())
			{
				// invalid bone index
				return false;
			}
			bool bNoFailures = true;
			for (int32 ConvexIdx : TransformToConvexInds[BoneIdx])
			{
				constexpr bool bConvertNonManifold = true; // Add non-manifold faces so we don't have holes messing up the sphere covering
				AppendConvexHullToCompactDynamicMesh(ConvexHulls[ConvexIdx].GetReference(), CombinedMesh, &GlobalTransformArray[BoneIdx], bConvertNonManifold);
			}
			return bNoFailures;
		};

		bool bNoFailures = true;
		for (int32 BoneIdx : UseSelection)
		{
			bool bSuccess = ProcessBone(BoneIdx);
			bNoFailures = bNoFailures && bSuccess;
		}

		UE::Geometry::FDynamicMeshAABBTree3 Tree(&CombinedMesh, true);
		UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3> Winding(&Tree, true);
		OutNegativeSpace.AddNegativeSpace(Winding, Settings, true);

		return bNoFailures;
	}
}
