// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineEdit.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"

#include "PlanarCut.h"


 
void FFractureEngineEdit::DeleteBranch(FGeometryCollection& GeometryCollection, const TArray<int32>& InBoneSelection)
{
	const FManagedArrayCollection& InCollection = (const FManagedArrayCollection&)GeometryCollection;
	GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

	TArray<int32> BoneIndicies = InBoneSelection;
	TransformSelectionFacade.RemoveRootNodes(BoneIndicies);
	TransformSelectionFacade.Sanitize(BoneIndicies);

	TArray<int32> NodesForDeletion;
	const TManagedArray<TSet<int32>>& Children = InCollection.GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);

	for (int32 BoneIdx : BoneIndicies)
	{
		FGeometryCollectionClusteringUtility::RecursiveAddAllChildren(Children, BoneIdx, NodesForDeletion);
	}

	NodesForDeletion.Sort();
	GeometryCollection.RemoveElements(FGeometryCollection::TransformGroup, NodesForDeletion);

	FGeometryCollectionClusteringUtility::RemoveDanglingClusters(&GeometryCollection);

	// Proximity is invalidated
	if (GeometryCollection.HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
	{
		GeometryCollection.RemoveAttribute("Proximity", FGeometryCollection::GeometryGroup);
	}
}


void FFractureEngineEdit::SetVisibilityInCollectionFromTransformSelection(FManagedArrayCollection& InCollection, const TArray<int32>& InTransformSelection, bool bVisible)
{
	GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

	const TManagedArray<int32>& TransformToGeometryIndex = InCollection.GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& FaceStart = InCollection.GetAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>& FaceCount = InCollection.GetAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
	TManagedArray<bool>& Visible = InCollection.ModifyAttribute<bool>("Visible", FGeometryCollection::FacesGroup);

	TArray<int32> TransformIndicies = InTransformSelection;
	TransformSelectionFacade.ConvertSelectionToRigidNodes(TransformIndicies);

	for (int32 Index : TransformIndicies)
	{
		// Iterate the faces in the geometry of this rigid node and set invisible.
		if (TransformToGeometryIndex[Index] > INDEX_NONE)
		{
			int32 CurrFace = FaceStart[TransformToGeometryIndex[Index]];
			for (int32 FaceOffset = 0; FaceOffset < FaceCount[TransformToGeometryIndex[Index]]; ++FaceOffset)
			{
				Visible[CurrFace + FaceOffset] = bVisible;
			}
		}
	}
}


void FFractureEngineEdit::SetVisibilityInCollectionFromFaceSelection(FManagedArrayCollection& InCollection, const TArray<int32>& InFaceSelection, bool bVisible)
{
	TManagedArray<bool>& Visible = InCollection.ModifyAttribute<bool>("Visible", FGeometryCollection::FacesGroup);

	int32 NumFaces = InCollection.NumElements(FGeometryCollection::FacesGroup);

	for (int32 FaceIdx : InFaceSelection)
	{
		Visible[FaceIdx] = bVisible;
	}
}


void FFractureEngineEdit::Merge(FGeometryCollection& GeometryCollection, const TArray<int32>& InBoneSelection)
{
	const FManagedArrayCollection& InCollection = (const FManagedArrayCollection&)GeometryCollection;
	GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

	TArray<int32> BoneIndicies = InBoneSelection;
	TransformSelectionFacade.Sanitize(BoneIndicies);

	const TArray<int32>& NodesForMerge = InBoneSelection;

	constexpr bool bBooleanUnion = false;
	MergeAllSelectedBones(GeometryCollection, NodesForMerge, bBooleanUnion);

	// Proximity is invalidated
	if (GeometryCollection.HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
	{
		GeometryCollection.RemoveAttribute("Proximity", FGeometryCollection::GeometryGroup);
	}
}


