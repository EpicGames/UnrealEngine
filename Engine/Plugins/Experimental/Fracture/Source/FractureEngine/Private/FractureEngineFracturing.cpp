// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineFracturing.h"
#include "FractureEngineSelection.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "Voronoi/Voronoi.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "Algo/RemoveIf.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "FractureEngineMaterials.h"
#include "Dataflow/DataflowSettings.h"

// Local helpers

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureEngineFracturing)
namespace UE::Private::FractureHelpers
{
	static void GenerateVoronoiSites(const FBox& InBoundingBox,
		const int32 InMinVoronoiSites,
		const int32 InMaxVoronoiSites,
		const int32 InRandomSeed,
		TArray<FVector>& OutSites)
	{
		FRandomStream RandStream(InRandomSeed);

		const FVector Extent(InBoundingBox.Max - InBoundingBox.Min);

		const int32 SiteCount = RandStream.RandRange(InMinVoronoiSites, InMaxVoronoiSites);

		OutSites.Reserve(OutSites.Num() + SiteCount);
		for (int32 ii = 0; ii < SiteCount; ++ii)
		{
			OutSites.Emplace(InBoundingBox.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
		}
	}

	static float GetMaxVertexMovement(const float InGrout,
		const float InAmplitude,
		const int32 InOctaveNumber,
		const float InPersistence)
	{
		float MaxDisp = InGrout;
		float AmplitudeScaled = InAmplitude;

		for (int32 OctaveIdx = 0; OctaveIdx < InOctaveNumber; OctaveIdx++, AmplitudeScaled *= InPersistence)
		{
			MaxDisp += FMath::Abs(AmplitudeScaled);
		}

		return MaxDisp;
	}

	static FBox GetVoronoiBounds(const FBox& InBoundingBox,
		const TArray<FVector>& Sites,
		const float InGrout,
		const float InAmplitude,
		const int32 InOctaveNumber,
		const float InPersistence)
	{
		FBox VoronoiBounds = InBoundingBox;
		if (Sites.Num() > 0)
		{
			VoronoiBounds += FBox(Sites);
		}

		return VoronoiBounds.ExpandBy(GetMaxVertexMovement(InGrout, InAmplitude, InOctaveNumber, InPersistence) + KINDA_SMALL_NUMBER);
	}

	static void GenerateTemporaryGuids(FManagedArrayCollection& InCollection, int32 InStartIdx, bool InForceInit)
	{
		bool bNeedsInit = false;
		if (!InCollection.HasAttribute("GUID", FTransformCollection::TransformGroup))
		{
			FManagedArrayCollection::FConstructionParameters Params(FName(""), false);
			InCollection.AddAttribute<FGuid>("GUID", FTransformCollection::TransformGroup, Params);
			bNeedsInit = true;
		}

		if (bNeedsInit || InForceInit)
		{
			TManagedArray<FGuid>& Guids = InCollection.ModifyAttribute<FGuid>("GUID", FTransformCollection::TransformGroup);
			for (int32 Idx = InStartIdx; Idx < Guids.Num(); ++Idx)
			{
				Guids[Idx] = FGuid::NewGuid();
			}
		}
	}

	static void ProcessNewlyFracturedBones(FGeometryCollection& OutGeomCollection, int32 FirstNewGeometryIndex, int32 NewInternalMaterialID = INDEX_NONE)
	{
		if (FirstNewGeometryIndex == INDEX_NONE)
		{
			return;
		}

		// Assign internal material
		if (NewInternalMaterialID > INDEX_NONE)
		{
			FFractureEngineMaterials::SetMaterialOnGeometryAfter(OutGeomCollection, FirstNewGeometryIndex, FFractureEngineMaterials::ETargetFaces::InternalFaces, NewInternalMaterialID);
			OutGeomCollection.ReindexMaterials();
		}

		// Generate GUIDs
		GenerateTemporaryGuids(OutGeomCollection, FirstNewGeometryIndex, true);
	}

	static int32 UniformFractureProc(
		TUniquePtr<FGeometryCollection>& OutGeomCollection,
		const TArray<int32>& InTransformSelectionArr,
		const FUniformFractureProcSettings& InUniformFractureProcSettings)
	{
		TArray<FVector> Sites;
		GenerateVoronoiSites(InUniformFractureProcSettings.BBox,
			InUniformFractureProcSettings.MinVoronoiSites,
			InUniformFractureProcSettings.MaxVoronoiSites,
			InUniformFractureProcSettings.RandomSeed,
			Sites);

		FBox VoronoiBounds = GetVoronoiBounds(InUniformFractureProcSettings.BBox,
			Sites,
			InUniformFractureProcSettings.Grout,
			InUniformFractureProcSettings.NoiseSettings.Amplitude,
			InUniformFractureProcSettings.NoiseSettings.Octaves,
			InUniformFractureProcSettings.NoiseSettings.Persistence);

		FVector Origin = InUniformFractureProcSettings.Transform.GetTranslation();
		for (FVector& Site : Sites)
		{
			Site -= Origin;
		}
		VoronoiBounds.Min -= Origin;
		VoronoiBounds.Max -= Origin;
		FVoronoiDiagram Voronoi(Sites, VoronoiBounds, .1f);

		FPlanarCells VoronoiPlanarCells = FPlanarCells(Sites, Voronoi);
		VoronoiPlanarCells.InternalSurfaceMaterials.NoiseSettings = InUniformFractureProcSettings.NoiseSettings;

		int32 FirstNewGeometryIndex = CutMultipleWithPlanarCells(VoronoiPlanarCells, 
			*OutGeomCollection, 
			InTransformSelectionArr, 
			InUniformFractureProcSettings.Grout,
			InUniformFractureProcSettings.AddSamplesForCollision ? InUniformFractureProcSettings.CollisionSampleSpacing : 0.0f,
			InUniformFractureProcSettings.RandomSeed, 
			InUniformFractureProcSettings.Transform, 
			true, 
			true, 
			nullptr, 
			Origin, 
			FIslandSplitSettings(InUniformFractureProcSettings.SplitIslands, InUniformFractureProcSettings.CloseVertexDistance, InUniformFractureProcSettings.VertexToSurfaceBridgeDistance));

		// failed to cut
		if (FirstNewGeometryIndex == INDEX_NONE)
		{
			return INDEX_NONE;
		}

		ProcessNewlyFracturedBones(*OutGeomCollection, FirstNewGeometryIndex, InUniformFractureProcSettings.InternalMaterialID);

		return FirstNewGeometryIndex;
	}

	static void SelectLeavesHelper(const FGeometryCollection& GeometryCollection, FDataflowTransformSelection& InOutTransformSelection, int32 BoneIdx)
	{
		if (!ensure(BoneIdx < InOutTransformSelection.Num() && GeometryCollection.SimulationType.IsValidIndex(BoneIdx)))
		{
			return;
		}
		if (GeometryCollection.SimulationType[BoneIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			InOutTransformSelection.SetNotSelected(BoneIdx);
			for (int32 ChildIdx : GeometryCollection.Children[BoneIdx])
			{
				SelectLeavesHelper(GeometryCollection, InOutTransformSelection, ChildIdx);
			}
		}
		else
		{
			InOutTransformSelection.SetSelected(BoneIdx);
		}
	}
	
	static void ConvertToLeafSelection(const FGeometryCollection& GeometryCollection, FDataflowTransformSelection& InOutTransformSelection)
	{
		if (!InOutTransformSelection.IsValidForCollection(GeometryCollection))
		{
			TArray<int32> ValidSelection = InOutTransformSelection.AsArrayValidated(GeometryCollection);
			InOutTransformSelection.InitFromArray(GeometryCollection, ValidSelection);
		}
		for (int32 BoneIdx = 0; BoneIdx < InOutTransformSelection.Num(); ++BoneIdx)
		{
			if (InOutTransformSelection.IsSelected(BoneIdx))
			{
				SelectLeavesHelper(GeometryCollection, InOutTransformSelection, BoneIdx);
			}
		}
	}
}


static void AddAdditionalAttributesIfRequired(FManagedArrayCollection& InOutCollection)
{
	FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(InOutCollection, -1);
}

static bool GetValidGeoCenter(FManagedArrayCollection& InOutCollection, 
	const TManagedArray<int32>& TransformToGeometryIndex, 
	const TArray<FTransform>& Transforms, 
	const TManagedArray<int32>& Parents,
	const TManagedArray<TSet<int32>>& Children, 
	const TManagedArray<FBox>& BoundingBoxes, 
	const TManagedArray<int32>& SimulationTypes,
	int32 TransformIndex, 
	FVector& OutGeoCenter)
{

	if (SimulationTypes[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_Rigid)
	{
		OutGeoCenter = Transforms[TransformIndex].TransformPosition(BoundingBoxes[TransformToGeometryIndex[TransformIndex]].GetCenter());

		return true;
	}
	else if (SimulationTypes[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_None) // ie this is embedded geometry
	{
		int32 Parent = Parents[TransformIndex];
		int32 ParentGeo = Parent != INDEX_NONE ? TransformToGeometryIndex[Parent] : INDEX_NONE;
		if (ensureMsgf(ParentGeo != INDEX_NONE, TEXT("Embedded geometry should always have a rigid geometry parent!  Geometry collection may be malformed.")))
		{
			OutGeoCenter = Transforms[Parents[TransformIndex]].TransformPosition(BoundingBoxes[ParentGeo].GetCenter());
		}
		else
		{
			return false; // no valid value to return
		}

		return true;
	}
	else
	{
		FVector AverageCenter;
		int32 ValidVectors = 0;
		for (int32 ChildIndex : Children[TransformIndex])
		{

			if (GetValidGeoCenter(InOutCollection, TransformToGeometryIndex, Transforms, Parents, Children, BoundingBoxes, SimulationTypes, ChildIndex, OutGeoCenter))
			{
				if (ValidVectors == 0)
				{
					AverageCenter = OutGeoCenter;
				}
				else
				{
					AverageCenter += OutGeoCenter;
				}
				++ValidVectors;
			}
		}

		if (ValidVectors > 0)
		{
			OutGeoCenter = AverageCenter / ValidVectors;
			return true;
		}
	}

	return false;
}

//
// TODO: Rewrite this using facades
//
void FFractureEngineFracturing::GenerateExplodedViewAttribute(FManagedArrayCollection& InOutCollection, const FVector& InScale, const float InUniformScale, const int32 InViewFractureLevel, const int32 InMaxFractureLevel)
{
	// Check if InOutCollection is not empty
	if (InOutCollection.HasAttribute(FTransformCollection::TransformAttribute, FGeometryCollection::TransformGroup))
	{
		InOutCollection.AddAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup, FManagedArrayCollection::FConstructionParameters(FName(), false));
		check(InOutCollection.HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup));

		TManagedArray<FVector3f>& ExplodedVectors = InOutCollection.ModifyAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup);
		const TManagedArray<FTransform3f>& Transforms = InOutCollection.GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& TransformToGeometryIndices = InOutCollection.GetAttribute<int32>(FGeometryCollection::TransformToGeometryIndexAttribute, FGeometryCollection::TransformGroup);
		const TManagedArray<FBox>& BoundingBoxes = InOutCollection.GetAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);

		// Make sure we have valid "Level"
		AddAdditionalAttributesIfRequired(InOutCollection);

		const TManagedArray<int32>& Levels = InOutCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FTransformCollection::TransformGroup);
		const TManagedArray<int32>& Parents = InOutCollection.GetAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
		const TManagedArray<TSet<int32>>& Children = InOutCollection.GetAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& SimulationTypes = InOutCollection.GetAttribute<int32>(FGeometryCollection::SimulationTypeAttribute, FGeometryCollection::TransformGroup);

		int32 MaxFractureLevel = InMaxFractureLevel;
		for (int32 Idx = 0, ni = Transforms.Num(); Idx < ni; ++Idx)
		{
			if (Levels[Idx] > MaxFractureLevel)
				MaxFractureLevel = Levels[Idx];
		}

		TArray<FTransform> TransformArr;
		GeometryCollectionAlgo::GlobalMatrices(Transforms, Parents, TransformArr);

		TArray<FVector> TransformedCenters;
		TransformedCenters.SetNumUninitialized(TransformArr.Num());

		int32 TransformsCount = 0;

		FVector Center(ForceInitToZero);
		for (int32 Idx = 0, ni = Transforms.Num(); Idx < ni; ++Idx)
		{
			ExplodedVectors[Idx] = FVector3f::ZeroVector;
			FVector GeoCenter;

			if (GetValidGeoCenter(InOutCollection, TransformToGeometryIndices, TransformArr, Parents, Children, BoundingBoxes, SimulationTypes, Idx, GeoCenter))
			{
				TransformedCenters[Idx] = GeoCenter;
				if ((InViewFractureLevel < 0) || Levels[Idx] == InViewFractureLevel)
				{
					Center += TransformedCenters[Idx];
					++TransformsCount;
				}
			}
		}

		Center /= TransformsCount;

		for (int Level = 1; Level <= MaxFractureLevel; Level++)
		{
			for (int32 Idx = 0, ni = TransformArr.Num(); Idx < ni; ++Idx)
			{
				if ((InViewFractureLevel < 0) || Levels[Idx] == InViewFractureLevel)
				{
					FVector ScaleVec = InScale * InUniformScale;
					ExplodedVectors[Idx] = (FVector3f)(TransformedCenters[Idx] - Center) * (FVector3f)ScaleVec;
				}
				else
				{
					if (Parents[Idx] > -1)
					{
						ExplodedVectors[Idx] = ExplodedVectors[Parents[Idx]];
					}
				}
			}
		}
	}
}


static float GetMaxVertexMovement(float Grout, float Amplitude, int OctaveNumber, float Persistence)
{
	float MaxDisp = Grout;
	float AmplitudeScaled = Amplitude;

	for (int32 OctaveIdx = 0; OctaveIdx < OctaveNumber; OctaveIdx++, AmplitudeScaled *= Persistence)
	{
		MaxDisp += FMath::Abs(AmplitudeScaled);
	}

	return MaxDisp;
}


static void RandomReduceSelection(FDataflowTransformSelection& InOutTransformSelection, int32 InRandomSeed, float InProbToKeep)
{
	FRandomStream RandStream(InRandomSeed);

	for (int32 BoneIdx = 0; BoneIdx < InOutTransformSelection.Num(); ++BoneIdx)
	{
		if (InOutTransformSelection.IsSelected(BoneIdx))
		{
			if (RandStream.GetFraction() >= InProbToKeep) // range does not include 1, so if ProbToKeep is 1 this will never remove
			{
				InOutTransformSelection.SetNotSelected(BoneIdx);
			}
		}
	}
}


int32 FFractureEngineFracturing::VoronoiFracture(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	TArray<FVector> InSites,
	const FTransform& InTransform,
	int32 InRandomSeed,
	float InChanceToFracture,
	FIslandSplitSettings IslandSplitSettings,
	float InGrout,
	float InAmplitude,
	float InFrequency,
	float InPersistence,
	float InLacunarity,
	int32 InOctaveNumber,
	float InPointSpacing,
	bool InAddSamplesForCollision,
	float InCollisionSampleSpacing)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		if (InSites.Num() > 0)
		{
			//
			// Compute BoundingBox for InCollection
			//
			FBox BoundingBox(ForceInit);

			if (InOutCollection.HasAttribute(FTransformCollection::TransformAttribute, FGeometryCollection::TransformGroup) &&
				InOutCollection.HasAttribute(FTransformCollection::ParentAttribute, FGeometryCollection::TransformGroup) &&
				InOutCollection.HasAttribute(FGeometryCollection::TransformIndexAttribute, FGeometryCollection::GeometryGroup) &&
				InOutCollection.HasAttribute(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup))
			{
				const TManagedArray<FTransform3f>& Transforms = InOutCollection.GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FGeometryCollection::TransformGroup);
				const TManagedArray<int32>& ParentIndices = InOutCollection.GetAttribute<int32>(FTransformCollection::ParentAttribute, FGeometryCollection::TransformGroup);
				const TManagedArray<int32>& TransformIndices = InOutCollection.GetAttribute<int32>(FGeometryCollection::TransformIndexAttribute, FGeometryCollection::GeometryGroup);
				const TManagedArray<FBox>& BoundingBoxes = InOutCollection.GetAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);

				TArray<FMatrix> TmpGlobalMatrices;
				GeometryCollectionAlgo::GlobalMatrices(Transforms, ParentIndices, TmpGlobalMatrices);

				if (TmpGlobalMatrices.Num() > 0)
				{
					for (int32 BoxIdx = 0; BoxIdx < BoundingBoxes.Num(); ++BoxIdx)
					{
						const int32 TransformIndex = TransformIndices[BoxIdx];
						BoundingBox += BoundingBoxes[BoxIdx].TransformBy(TmpGlobalMatrices[TransformIndex]);
					}
				}

				FVector Origin = InTransform.GetTranslation();
				for (FVector& Site : InSites)
				{
					Site -= Origin;
				}

				//
				// Compute Voronoi Bounds
				//
				FBox VoronoiBounds = BoundingBox;
				VoronoiBounds += FBox(InSites);

				VoronoiBounds = VoronoiBounds.ExpandBy(GetMaxVertexMovement(InGrout, InAmplitude, InOctaveNumber, InPersistence) + KINDA_SMALL_NUMBER);

				//
				// Voronoi Fracture
				//
				FNoiseSettings NoiseSettings;
				NoiseSettings.Amplitude = InAmplitude;
				NoiseSettings.Frequency = InFrequency;
				NoiseSettings.Octaves = InOctaveNumber;
				NoiseSettings.PointSpacing = InPointSpacing;
				NoiseSettings.Lacunarity = InLacunarity;
				NoiseSettings.Persistence = InPersistence;

				FVoronoiDiagram Voronoi(InSites, VoronoiBounds, .1f);
				
				FPlanarCells VoronoiPlanarCells = FPlanarCells(InSites, Voronoi);
				VoronoiPlanarCells.InternalSurfaceMaterials.NoiseSettings = NoiseSettings;

				RandomReduceSelection(InTransformSelection, InRandomSeed, InChanceToFracture);
				
				UE::Private::FractureHelpers::ConvertToLeafSelection(*GeomCollection, InTransformSelection);
				TArray<int32> TransformSelectionArr = InTransformSelection.AsArrayValidated(*GeomCollection);

				if (!FFractureEngineSelection::IsBoneSelectionValid(InOutCollection, TransformSelectionArr))
				{
					return INDEX_NONE;
				}
				
				int ResultGeometryIndex = CutMultipleWithPlanarCells(VoronoiPlanarCells, *GeomCollection, TransformSelectionArr, InGrout, InAddSamplesForCollision ? InCollisionSampleSpacing : 0.0f, InRandomSeed, InTransform, true, true, nullptr, Origin, IslandSplitSettings);

				UE::Private::FractureHelpers::ProcessNewlyFracturedBones(*GeomCollection, ResultGeometryIndex);

				InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);

				return ResultGeometryIndex;
			}
		}
	}

	return INDEX_NONE;
}


void FFractureEngineFracturing::GenerateSliceTransforms(const FBox& InBoundingBox,
	const int32 InRandomSeed,
	const int32 InNumPlanes,
	TArray<FTransform>& OutCuttingPlaneTransforms)
{
	FRandomStream RandStream(InRandomSeed);

	FBox Bounds = InBoundingBox;
	const FVector Extent(Bounds.Max - Bounds.Min);

	OutCuttingPlaneTransforms.Reserve(OutCuttingPlaneTransforms.Num() + InNumPlanes);
	for (int32 Idx = 0; Idx < InNumPlanes; ++Idx)
	{
		FVector Position(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
		OutCuttingPlaneTransforms.Emplace(FTransform(FRotator(RandStream.FRand() * 360.0f, RandStream.FRand() * 360.0f, 0.0f), Position));
	}
}

int32 FFractureEngineFracturing::PlaneCutter(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	const FBox& InBoundingBox,
	const FTransform& InTransform,
	int32 InNumPlanes,
	int32 InRandomSeed,
	float InChanceToFracture,
	FIslandSplitSettings IslandSplitSettings,
	float InGrout,
	float InAmplitude,
	float InFrequency,
	float InPersistence,
	float InLacunarity,
	int32 InOctaveNumber,
	float InPointSpacing,
	bool InAddSamplesForCollision,
	float InCollisionSampleSpacing,
	TConstArrayView<FTransform> InCutPlaneTransforms)
{

	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		TArray<FPlane> CuttingPlanes;
		TArray<FTransform> CuttingPlaneTransforms(InCutPlaneTransforms);

		FFractureEngineFracturing::GenerateSliceTransforms(InBoundingBox, InRandomSeed, InNumPlanes, CuttingPlaneTransforms);

		for (const FTransform& Transform : CuttingPlaneTransforms)
		{
			CuttingPlanes.Add(FPlane(Transform.GetLocation(), Transform.GetUnitAxis(EAxis::Z)));
		}

		FInternalSurfaceMaterials InternalSurfaceMaterials;
		FNoiseSettings NoiseSettings;

		if (InAmplitude > 0.f)
		{
			NoiseSettings.Amplitude = InAmplitude;
			NoiseSettings.Frequency = InFrequency;
			NoiseSettings.Lacunarity = InLacunarity;
			NoiseSettings.Persistence = InPersistence;
			NoiseSettings.Octaves = InOctaveNumber;
			NoiseSettings.PointSpacing = InPointSpacing;

			InternalSurfaceMaterials.NoiseSettings = NoiseSettings;
		}

		float CollisionSampleSpacingVal = InAddSamplesForCollision ? InCollisionSampleSpacing : 0.0f;
		float GroutVal = InGrout;

		RandomReduceSelection(InTransformSelection, InRandomSeed, InChanceToFracture);
		UE::Private::FractureHelpers::ConvertToLeafSelection(*GeomCollection, InTransformSelection);
		TArray<int32> TransformSelectionArr = InTransformSelection.AsArrayValidated(*GeomCollection);

		if (!FFractureEngineSelection::IsBoneSelectionValid(InOutCollection, TransformSelectionArr))
		{
			return INDEX_NONE;
		}

		int ResultGeometryIndex = CutMultipleWithMultiplePlanes(CuttingPlanes, InternalSurfaceMaterials, *GeomCollection, TransformSelectionArr, GroutVal, CollisionSampleSpacingVal, InRandomSeed, InTransform, true, nullptr, IslandSplitSettings);

		UE::Private::FractureHelpers::ProcessNewlyFracturedBones(*GeomCollection, ResultGeometryIndex);

		InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);

		return ResultGeometryIndex;
	}

	return INDEX_NONE;
}

void FFractureEngineFracturing::GenerateSliceTransforms(TArray<FTransform>& InOutCuttingPlaneTransforms, 
	const FBox& InBoundingBox,
	int32 InSlicesX,
	int32 InSlicesY,
	int32 InSlicesZ,
	int32 InRandomSeed,
	float InSliceAngleVariation,
	float InSliceOffsetVariation)
{
	const FBox& Bounds = InBoundingBox;
	const FVector& Min = Bounds.Min;
	const FVector& Max = Bounds.Max;
	const FVector  Center = Bounds.GetCenter();
	const FVector Extents(Max - Min);
	const FVector HalfExtents(Extents * 0.5f);

	const FVector Step(Extents.X / (InSlicesX + 1), Extents.Y / (InSlicesY + 1), Extents.Z / (InSlicesZ + 1));

	InOutCuttingPlaneTransforms.Reserve(InSlicesX * InSlicesY * InSlicesZ);

	FRandomStream RandomStream(InRandomSeed);

	const float SliceAngleVariationInRadians = FMath::DegreesToRadians(InSliceAngleVariation);

	const FVector XMin(-Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));
	const FVector XMax(Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));

	for (int32 xx = 0; xx < InSlicesX; ++xx)
	{
		const FVector SlicePosition(FVector(Min.X, Center.Y, Center.Z) + FVector((Step.X * xx) + Step.X, 0.0f, 0.0f) + RandomStream.VRand() * RandomStream.GetFraction() * InSliceOffsetVariation);
		FTransform Transform(FQuat(FVector::RightVector, FMath::DegreesToRadians(90)), SlicePosition);
		const FQuat RotA(FVector::RightVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		const FQuat RotB(FVector::ForwardVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		Transform.ConcatenateRotation(FQuat(RotA * RotB));
		InOutCuttingPlaneTransforms.Emplace(Transform);
	}

	const FVector YMin(-Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));
	const FVector YMax(Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));

	for (int32 yy = 0; yy < InSlicesY; ++yy)
	{
		const FVector SlicePosition(FVector(Center.X, Min.Y, Center.Z) + FVector(0.0f, (Step.Y * yy) + Step.Y, 0.0f) + RandomStream.VRand() * RandomStream.GetFraction() * InSliceOffsetVariation);
		FTransform Transform(FQuat(FVector::ForwardVector, FMath::DegreesToRadians(90)), SlicePosition);
		const FQuat RotA(FVector::RightVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		const FQuat RotB(FVector::ForwardVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		Transform.ConcatenateRotation(RotA * RotB);
		InOutCuttingPlaneTransforms.Emplace(Transform);
	}

	const FVector ZMin(-Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));
	const FVector ZMax(Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));

	for (int32 zz = 0; zz < InSlicesZ; ++zz)
	{
		const FVector SlicePosition(FVector(Center.X, Center.Y, Min.Z) + FVector(0.0f, 0.0f, (Step.Z * zz) + Step.Z) + RandomStream.VRand() * RandomStream.GetFraction() * InSliceOffsetVariation);
		FTransform Transform(SlicePosition);
		const FQuat RotA(FVector::RightVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		const FQuat RotB(FVector::ForwardVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		Transform.ConcatenateRotation(RotA * RotB);
		InOutCuttingPlaneTransforms.Emplace(Transform);
	}
}

static void ClearProximity(FGeometryCollection* GeometryCollection)
{
	if (GeometryCollection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
	{
		GeometryCollection->RemoveAttribute("Proximity", FGeometryCollection::GeometryGroup);
	}
}

int32 FFractureEngineFracturing::SliceCutter(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	const FBox& InBoundingBox,
	int32 InSlicesX,
	int32 InSlicesY,
	int32 InSlicesZ,
	float InSliceAngleVariation,
	float InSliceOffsetVariation,
	int32 InRandomSeed,
	float InChanceToFracture,
	FIslandSplitSettings IslandSplitSettings,
	float InGrout,
	float InAmplitude,
	float InFrequency,
	float InPersistence,
	float InLacunarity,
	int32 InOctaveNumber,
	float InPointSpacing,
	bool InAddSamplesForCollision,
	float InCollisionSampleSpacing)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		TArray<FTransform> LocalCuttingPlanesTransforms;
		GenerateSliceTransforms(LocalCuttingPlanesTransforms,
			InBoundingBox,
			InSlicesX,
			InSlicesY,
			InSlicesZ,
			InRandomSeed,
			InSliceAngleVariation,
			InSliceOffsetVariation);

		TArray<FPlane> CuttingPlanes;
		CuttingPlanes.Reserve(LocalCuttingPlanesTransforms.Num());

		for (const FTransform& Transform : LocalCuttingPlanesTransforms)
		{
			CuttingPlanes.Add(FPlane(Transform.GetLocation(), Transform.GetUnitAxis(EAxis::Z)));
		}

		FInternalSurfaceMaterials InternalSurfaceMaterials;
		FNoiseSettings NoiseSettings;

		if (InAmplitude > 0.f)
		{
			NoiseSettings.Amplitude = InAmplitude;
			NoiseSettings.Frequency = InFrequency;
			NoiseSettings.Lacunarity = InLacunarity;
			NoiseSettings.Persistence = InPersistence;
			NoiseSettings.Octaves = InOctaveNumber;
			NoiseSettings.PointSpacing = InPointSpacing;

			InternalSurfaceMaterials.NoiseSettings = NoiseSettings;
		}

		float CollisionSampleSpacingVal = InAddSamplesForCollision ? InCollisionSampleSpacing : 0.0f;
		float GroutVal = InGrout;

		RandomReduceSelection(InTransformSelection, InRandomSeed, InChanceToFracture);
		UE::Private::FractureHelpers::ConvertToLeafSelection(*GeomCollection, InTransformSelection);
		TArray<int32> TransformSelectionArr = InTransformSelection.AsArrayValidated(*GeomCollection);

		if (!FFractureEngineSelection::IsBoneSelectionValid(InOutCollection, TransformSelectionArr))
		{
			return INDEX_NONE;
		}

		// Proximity is invalidated.
		ClearProximity(GeomCollection.Get());

		int ResultGeometryIndex = CutMultipleWithMultiplePlanes(CuttingPlanes, InternalSurfaceMaterials, *GeomCollection, TransformSelectionArr, GroutVal, CollisionSampleSpacingVal, InRandomSeed, FTransform::Identity, true, nullptr, IslandSplitSettings);

		UE::Private::FractureHelpers::ProcessNewlyFracturedBones(*GeomCollection, ResultGeometryIndex);

		InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);

		return ResultGeometryIndex;
	}

	return INDEX_NONE;
}

namespace FractureToolBrickLocals
{
	// Calculate total number of bricks based on given dimensions and the extent of the object to be fractured.
	// If the input is not valid or the result is too large, this functions returns -1.
	// It is possible that we are dealing with incredibly large meshes and small brick dimensions. Doing the
	// calculations in double and checking for NaNs will catch cases where for integer we would have to jump through
	// multiple hoops to make sure we are not dealing with overflow. And since the limit for the number of bricks is
	// comparably low, we do not need to worry about loss in precision for very large integers.
	// For example, running this calculation with brick dimensions of 1 for the Sky Sphere will result in integer
	// overflow.
	static int64 CalculateNumBricks(const FVector& Dimensions, const FVector& Extents)
	{
		if (Dimensions.GetMin() <= 0 || Extents.GetMin() <= 0)
		{
			return -1;
		}

		const FVector NumBricksPerDim(ceil(Extents.X / Dimensions.X), ceil(Extents.Y / Dimensions.Y), ceil(Extents.Z / Dimensions.Z));
		if (NumBricksPerDim.ContainsNaN())
		{
			return -1;
		}

		const double NumBricks = NumBricksPerDim.X * NumBricksPerDim.Y * NumBricksPerDim.Z;
		if (FMath::IsNaN(NumBricks))
		{
			return -1;
		}

		return static_cast<int64>(NumBricks);
	}

	static FVector GetBrickDimensions(const float InBrickLength,
		const float InBrickHeight,
		const float InBrickDepth, 
		const FVector& InExtents)
	{
		// Limit for the total number of bricks.
		const int64 NumBricksLimit = 8192;

		FVector Dimensions(InBrickLength, InBrickDepth, InBrickHeight);

		// Early out if we have inputs we cannot deal with. If this call to CalculateNumBricks is fine then any other
		// call to it will be fine, too, and we do not need to check for invalid results again.
		int64 NumBricks = CalculateNumBricks(Dimensions, InExtents);
		if (NumBricks < 0)
		{
			return FVector::ZeroVector;
		}

		if (NumBricks > NumBricksLimit)
		{
			const int64 InputNumBricks = NumBricks;

			// Determine dimensions safely within the brick limit by iteratively doubling the brick size.
			FVector SafeDimensions = Dimensions;
			int64 SafeNumBricks;
			do
			{
				SafeDimensions *= 2;
				SafeNumBricks = CalculateNumBricks(SafeDimensions, InExtents);
			} while (SafeNumBricks > NumBricksLimit);

			// Maximize brick dimensions to fit within the brick limit via iterative interval halving.
			const int32 IterationsMax = 10;
			int32 Iterations = 0;
			do
			{
				const FVector MidDimensions = (Dimensions + SafeDimensions) / 2;
				const int64 MidNumBricks = CalculateNumBricks(MidDimensions, InExtents);

				if (MidNumBricks > NumBricksLimit)
				{
					Dimensions = MidDimensions;
					NumBricks = MidNumBricks;
				}
				else
				{
					SafeDimensions = MidDimensions;
					SafeNumBricks = MidNumBricks;
				}
			} while (++Iterations < IterationsMax);

			Dimensions = SafeDimensions;
			NumBricks = SafeNumBricks;

//			UE_LOG(LogFractureTool, Warning, TEXT("Brick Fracture: Current brick dimensions of %f x %f x %f would result in %d bricks. "
//				"Reduced brick dimensions to %f x %f x %f resulting in %d bricks to stay within maximum number of %d bricks."),
//				InBrickLength, InBrickDepth, InBrickHeight, InputNumBricks,
//				Dimensions.X, Dimensions.Y, Dimensions.Z, NumBricks, NumBricksLimit);
		}

		return Dimensions;
	}
}

void FFractureEngineFracturing::AddBoxEdges(TArray<TTuple<FVector, FVector>>& InOutEdges, const FVector& InMin, const FVector& InMax)
{
	InOutEdges.Emplace(MakeTuple(InMin, FVector(InMin.X, InMax.Y, InMin.Z)));
	InOutEdges.Emplace(MakeTuple(InMin, FVector(InMin.X, InMin.Y, InMax.Z)));
	InOutEdges.Emplace(MakeTuple(FVector(InMin.X, InMax.Y, InMax.Z), FVector(InMin.X, InMax.Y, InMin.Z)));
	InOutEdges.Emplace(MakeTuple(FVector(InMin.X, InMax.Y, InMax.Z), FVector(InMin.X, InMin.Y, InMax.Z)));

	InOutEdges.Emplace(MakeTuple(FVector(InMax.X, InMin.Y, InMin.Z), FVector(InMax.X, InMax.Y, InMin.Z)));
	InOutEdges.Emplace(MakeTuple(FVector(InMax.X, InMin.Y, InMin.Z), FVector(InMax.X, InMin.Y, InMax.Z)));
	InOutEdges.Emplace(MakeTuple(InMax, FVector(InMax.X, InMax.Y, InMin.Z)));
	InOutEdges.Emplace(MakeTuple(InMax, FVector(InMax.X, InMin.Y, InMax.Z)));

	InOutEdges.Emplace(MakeTuple(InMin, FVector(InMax.X, InMin.Y, InMin.Z)));
	InOutEdges.Emplace(MakeTuple(FVector(InMin.X, InMin.Y, InMax.Z), FVector(InMax.X, InMin.Y, InMax.Z)));
	InOutEdges.Emplace(MakeTuple(FVector(InMin.X, InMax.Y, InMin.Z), FVector(InMax.X, InMax.Y, InMin.Z)));
	InOutEdges.Emplace(MakeTuple(FVector(InMin.X, InMax.Y, InMax.Z), InMax));
}

void FFractureEngineFracturing::GenerateBrickTransforms(const FBox& InBounds, 
	TArray<FTransform>& InOutBrickTransforms, 
	const EFractureBrickBondEnum InBond,
	const float InBrickLength,
	const float InBrickHeight,
	const float InBrickDepth,
	TArray<TTuple<FVector, FVector>>& InOutEdges)
{
	// Determine brick dimensions (length, depth, height) and make sure we do not exceed the limit for the number of bricks.
	// If we would simply use the input dimensions, we are prone to running out of memory and/or exceeding the storage capabilities of TArray, and crash.
	const FVector BrickDimensions = FractureToolBrickLocals::GetBrickDimensions(InBrickLength, InBrickHeight, InBrickDepth, InBounds.Max - InBounds.Min);
	const FVector BrickHalfDimensions(BrickDimensions * 0.5);

	// make conservative version of dims to account for 90-degree brick rotation in some patterns
	FVector ConservativeHalfBrick = BrickHalfDimensions;
	if (InBond != EFractureBrickBondEnum::Dataflow_FractureBrickBond_Stretcher && InBond != EFractureBrickBondEnum::Dataflow_FractureBrickBond_Stack)
	{
		double MaxXY = FMath::Max(ConservativeHalfBrick.X, ConservativeHalfBrick.Y);
		ConservativeHalfBrick.X = MaxXY;
		ConservativeHalfBrick.Y = MaxXY;
	}

	const FVector Min = InBounds.Min;
	// Bricks are tiled up to the point where their center is beyond Max in any dimension, so
	// to cover InBounds, we need this Max to be at least a half-brick beyond the InBounds.Max
	const FVector Max = InBounds.Max + ConservativeHalfBrick + FVector(UE_DOUBLE_KINDA_SMALL_NUMBER);
	const FVector Extents(Max - Min);

	// Early out if we have inputs we cannot deal with.
	if (BrickDimensions == FVector::ZeroVector)
	{
		return;
	}

	// Reserve correct amount of memory to avoid re-allocations.
	InOutBrickTransforms.Reserve(FractureToolBrickLocals::CalculateNumBricks(BrickDimensions, Extents));

	const FQuat HeaderRotation(FVector::UpVector, UE_DOUBLE_HALF_PI);

	if (InBond == EFractureBrickBondEnum::Dataflow_FractureBrickBond_Stretcher)
	{
		bool OddY = false;
		for (float yy = 0.f; yy <= Extents.Y; yy += BrickDimensions.Y)
		{
			bool Oddline = false;
			for (float zz = BrickHalfDimensions.Z; zz <= Extents.Z; zz += BrickDimensions.Z)
			{
				for (float xx = 0.f; xx <= Extents.X; xx += BrickDimensions.X)
				{
					FVector BrickPosition(Min + FVector(Oddline ^ OddY ? xx : xx + BrickHalfDimensions.X, yy, zz));
					InOutBrickTransforms.Emplace(FTransform(BrickPosition));
				}
				Oddline = !Oddline;
			}
			OddY = !OddY;
		}
	}
	else if (InBond == EFractureBrickBondEnum::Dataflow_FractureBrickBond_Stack)
	{
		bool OddY = false;
		for (float yy = 0.f; yy <= Extents.Y; yy += BrickDimensions.Y)
		{
			for (float zz = BrickHalfDimensions.Z; zz <= Extents.Z; zz += BrickDimensions.Z)
			{
				for (float xx = 0.f; xx <= Extents.X; xx += BrickDimensions.X)
				{
					FVector BrickPosition(Min + FVector(OddY ? xx : xx + BrickHalfDimensions.X, yy, zz));
					InOutBrickTransforms.Emplace(FTransform(BrickPosition));
				}
			}
			OddY = !OddY;
		}
	}
	else if (InBond == EFractureBrickBondEnum::Dataflow_FractureBrickBond_English)
	{
		float HalfLengthDepthDifference = BrickHalfDimensions.X - BrickHalfDimensions.Y - BrickHalfDimensions.Y;
		bool OddY = false;
		for (float yy = 0.f; yy <= Extents.Y; yy += BrickDimensions.Y)
		{
			bool Oddline = false;
			for (float zz = BrickHalfDimensions.Z; zz <= Extents.Z; zz += BrickDimensions.Z)
			{
				if (Oddline && !OddY) // header row
				{
					for (float xx = 0.f; xx <= Extents.X; xx += BrickDimensions.Y)
					{
						FVector BrickPosition(Min + FVector(Oddline ^ OddY ? xx : xx + BrickHalfDimensions.Y, yy + BrickHalfDimensions.Y, zz));
						InOutBrickTransforms.Emplace(FTransform(HeaderRotation, BrickPosition));
					}
				}
				else if (!Oddline) // stretchers
				{
					for (float xx = 0.f; xx <= Extents.X; xx += BrickDimensions.X)
					{
						FVector BrickPosition(Min + FVector(Oddline ^ OddY ? xx : xx + BrickHalfDimensions.X, OddY ? yy + HalfLengthDepthDifference : yy - HalfLengthDepthDifference, zz));
						InOutBrickTransforms.Emplace(FTransform(BrickPosition));
					}
				}
				Oddline = !Oddline;
			}
			OddY = !OddY;
		}
	}
	else if (InBond == EFractureBrickBondEnum::Dataflow_FractureBrickBond_Header)
	{
		bool OddY = false;
		for (float yy = 0.f; yy <= Extents.Y; yy += BrickDimensions.X)
		{
			bool Oddline = false;
			for (float zz = BrickHalfDimensions.Z; zz <= Extents.Z; zz += BrickDimensions.Z)
			{
				for (float xx = 0.f; xx <= Extents.X; xx += BrickDimensions.Y)
				{
					FVector BrickPosition(Min + FVector(Oddline ^ OddY ? xx : xx + BrickHalfDimensions.Y, yy, zz));
					InOutBrickTransforms.Emplace(FTransform(HeaderRotation, BrickPosition));
				}
				Oddline = !Oddline;
			}
			OddY = !OddY;
		}
	}
	else if (InBond == EFractureBrickBondEnum::Dataflow_FractureBrickBond_Flemish)
	{
		float HalfLengthDepthDifference = BrickHalfDimensions.X - BrickDimensions.Y;
		bool OddY = false;
		int32 RowY = 0;
		for (float yy = 0.f; yy <= Extents.Y; yy += BrickDimensions.Y)
		{
			bool OddZ = false;
			for (float zz = BrickHalfDimensions.Z; zz <= Extents.Z; zz += BrickDimensions.Z)
			{
				bool OddX = OddZ;
				for (float xx = 0.f; xx <= Extents.X; xx += BrickHalfDimensions.X + BrickHalfDimensions.Y)
				{
					FVector BrickPosition(Min + FVector(xx, yy, zz));
					if (OddX)
					{
						if (OddY) // runner
						{
							InOutBrickTransforms.Emplace(FTransform(BrickPosition + FVector(0, HalfLengthDepthDifference, 0))); // runner
						}
						else
						{
							InOutBrickTransforms.Emplace(FTransform(BrickPosition - FVector(0, HalfLengthDepthDifference, 0))); // runner

						}
					}
					else if (!OddY) // header
					{
						InOutBrickTransforms.Emplace(FTransform(HeaderRotation, BrickPosition + FVector(0, BrickHalfDimensions.Y, 0))); // header
					}
					OddX = !OddX;
				}
				OddZ = !OddZ;
			}
			OddY = !OddY;
			++RowY;
		}
	}

	const FVector BrickMax(BrickHalfDimensions);
	const FVector BrickMin(-BrickHalfDimensions);

	for (const auto& Transform : InOutBrickTransforms)
	{
		AddBoxEdges(InOutEdges, Transform.TransformPosition(BrickMin), Transform.TransformPosition(BrickMax));
	}
}

int32 FFractureEngineFracturing::BrickCutter(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	const FBox& InBoundingBox,
	const FTransform& InTransform,
	EFractureBrickBondEnum InBond,
	float InBrickLength,
	float InBrickHeight,
	float InBrickDepth,
	int32 InRandomSeed,
	float InChanceToFracture,
	FIslandSplitSettings IslandSplitSettings,
	float InGrout,
	float InAmplitude,
	float InFrequency,
	float InPersistence,
	float InLacunarity,
	int32 InOctaveNumber,
	float InPointSpacing,
	bool InAddSamplesForCollision,
	float InCollisionSampleSpacing)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		TArray<FTransform> BrickTransforms;
		TArray<TTuple<FVector, FVector>> Edges;
		TArray<TTuple<FVector, FVector>> Boxes;

		BrickTransforms.Empty();

		const FBox Bounds = InBoundingBox;
		GenerateBrickTransforms(Bounds,
			BrickTransforms,
			InBond,
			InBrickLength,
			InBrickHeight,
			InBrickDepth,			
			Edges);

		// Get the same brick dimensions that were used in GenerateBrickTransform.
		// If we cannot deal with the input data then the brick dimensions will be zero, but we do not need to
		// explicitly handle that since it will only affect some local variables. The BrickTransforms will be empty
		// and there are no further side effects.
		const FVector BrickDimensions = FractureToolBrickLocals::GetBrickDimensions(InBrickLength, InBrickHeight, InBrickDepth, Bounds.Max - Bounds.Min);
		const FVector BrickHalfDimensions(BrickDimensions * 0.5);

		TArray<FBox> BricksToCut;

		// space the bricks by the grout setting, constrained to not erase the bricks
		const float MinDim = FMath::Min3(BrickHalfDimensions.X, BrickHalfDimensions.Y, BrickHalfDimensions.Z);
		const float HalfGrout = FMath::Clamp(0.5f * InGrout, 0, MinDim * 0.98f);
		const FVector HalfBrick(BrickHalfDimensions - HalfGrout);
		const FBox BrickBox(-HalfBrick, HalfBrick);

		FTransform ContextTransform = InTransform;
		FVector Origin = ContextTransform.GetTranslation();

		for (const FTransform& Trans : BrickTransforms)
		{
			FTransform ToApply = Trans * FTransform(-Origin);
			BricksToCut.Add(BrickBox.TransformBy(ToApply));
		}

		FInternalSurfaceMaterials InternalSurfaceMaterials;
		FNoiseSettings NoiseSettings;

		if (InAmplitude > 0.f)
		{
			NoiseSettings.Amplitude = InAmplitude;
			NoiseSettings.Frequency = InFrequency;
			NoiseSettings.Lacunarity = InLacunarity;
			NoiseSettings.Persistence = InPersistence;
			NoiseSettings.Octaves = InOctaveNumber;
			NoiseSettings.PointSpacing = InPointSpacing;

			InternalSurfaceMaterials.NoiseSettings = NoiseSettings;
		}

		float CollisionSampleSpacingVal = InAddSamplesForCollision ? InCollisionSampleSpacing : 0.0f;

		RandomReduceSelection(InTransformSelection, InRandomSeed, InChanceToFracture);
		UE::Private::FractureHelpers::ConvertToLeafSelection(*GeomCollection, InTransformSelection);
		TArray<int32> TransformSelectionArr = InTransformSelection.AsArrayValidated(*GeomCollection);

		if (!FFractureEngineSelection::IsBoneSelectionValid(InOutCollection, TransformSelectionArr))
		{
			return INDEX_NONE;
		}

		float GroutVal = 0.f; // CutterSettings->Grout; // Note: Grout is currently baked directly into the brick cells above
		const bool bBricksAreTouching = InGrout <= UE_KINDA_SMALL_NUMBER;
		FPlanarCells Cells = FPlanarCells(BricksToCut, bBricksAreTouching);

		int32 ResultGeometryIndex = CutMultipleWithPlanarCells(Cells, *GeomCollection, TransformSelectionArr, GroutVal, CollisionSampleSpacingVal, InRandomSeed, InTransform, true, true, nullptr, Origin, IslandSplitSettings);

		UE::Private::FractureHelpers::ProcessNewlyFracturedBones(*GeomCollection, ResultGeometryIndex);

		InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);

		return ResultGeometryIndex;
	}

	return INDEX_NONE;
}
	
void FFractureEngineFracturing::GenerateMeshTransforms(TArray<FTransform>& MeshTransforms,
	const FBox& InBoundingBox,
	const int32 InRandomSeed,
	const EMeshCutterCutDistribution InCutDistribution,
	const int32 InNumberToScatter,
	const int32 InGridX,
	const int32 InGridY,
	const int32 InGridZ,
	const float InVariability,
	const float InMinScaleFactor,
	const float InMaxScaleFactor,
	const bool InRandomOrientation,
	const float InRollRange,
	const float InPitchRange,
	const float InYawRange)
{
	FRandomStream RandStream(InRandomSeed);

	FBox Bounds = InBoundingBox;
	const FVector Extent(Bounds.Max - Bounds.Min);

	TArray<FVector> Positions;
	if (InCutDistribution == EMeshCutterCutDistribution::UniformRandom)
	{
		Positions.Reserve(InNumberToScatter);
		for (int32 Idx = 0; Idx < InNumberToScatter; ++Idx)
		{
			Positions.Emplace(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
		}
	}
	else if (InCutDistribution == EMeshCutterCutDistribution::Grid)
	{
		Positions.Reserve(InGridX * InGridY * InGridZ);
		auto ToFrac = [](int32 Val, int32 NumVals) -> FVector::FReal
		{
			return (FVector::FReal(Val) + FVector::FReal(.5)) / FVector::FReal(NumVals);
		};
		for (int32 X = 0; X < InGridX; ++X)
		{
			FVector::FReal XFrac = ToFrac(X, InGridX);
			for (int32 Y = 0; Y < InGridY; ++Y)
			{
				FVector::FReal YFrac = ToFrac(Y, InGridY);
				for (int32 Z = 0; Z < InGridZ; ++Z)
				{
					FVector::FReal ZFrac = ToFrac(Z, InGridZ);
					Positions.Emplace(Bounds.Min + FVector(XFrac, YFrac, ZFrac) * Extent);
				}
			}
		}

		for (FVector& Position : Positions)
		{
			Position += (RandStream.VRand() * RandStream.FRand() * InVariability);
		}
	}

	MeshTransforms.Reserve(MeshTransforms.Num() + Positions.Num());
	for (const FVector& Position : Positions)
	{
		const FVector ScaleVec(RandStream.FRandRange(InMinScaleFactor, InMaxScaleFactor));
		FRotator Orientation = FRotator::ZeroRotator;
		if (InRandomOrientation)
		{
			Orientation = FRotator(
				RandStream.FRandRange(-InPitchRange, InPitchRange),
				RandStream.FRandRange(-InYawRange, InYawRange),
				RandStream.FRandRange(-InRollRange, InRollRange)
			);
		}
		MeshTransforms.Emplace(FTransform(Orientation, Position, ScaleVec));
	}
}

int32 FFractureEngineFracturing::MeshArrayCutter(TArray<FTransform>& MeshTransforms,
	FManagedArrayCollection& InOutCollection,
	const FDataflowTransformSelection& InTransformSelectionConst,
	TArrayView<const UE::Geometry::FDynamicMesh3*> InDynCuttingMeshes,
	const EMeshCutterPerCutMeshSelection PerCutMeshSelection,
	const int32 InRandomSeed,
	const float InChanceToFracture,
	const FIslandSplitSettings IslandSplitSettings,
	const float InCollisionSampleSpacing)
{

	if (InOutCollection.NumElements(FTransformCollection::TransformGroup) == 0)
	{
		// empty collection, early exit
		return -1;
	}

	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		// Note: Noise not currently supported
		FInternalSurfaceMaterials InternalSurfaceMaterials;

		int32 ResultGeometryIndex = -1;

		const int32 OriginalNumTransforms = InOutCollection.NumElements(FGeometryCollection::TransformGroup);

		FRandomStream RandStream(InRandomSeed);

		FDataflowTransformSelection InTransformSelection = InTransformSelectionConst;
		RandomReduceSelection(InTransformSelection, InRandomSeed, InChanceToFracture);
		UE::Private::FractureHelpers::ConvertToLeafSelection(*GeomCollection, InTransformSelection);
		TArray<int32> TransformSelectionArr = InTransformSelection.AsArrayValidated(*GeomCollection);

		int32 SequentialMeshIndex = 0;
		for (const FTransform& ScatterTransform : MeshTransforms)
		{
			auto ApplyCut = 
				[&ScatterTransform, &InternalSurfaceMaterials, &GeomCollection, &TransformSelectionArr, InCollisionSampleSpacing, IslandSplitSettings, &ResultGeometryIndex]
				(const UE::Geometry::FDynamicMesh3& CuttingMesh)
			{
				constexpr bool bSetDefaultInternalMaterialsFromCollection = true;
				int32 Index = CutWithMesh(CuttingMesh, ScatterTransform, InternalSurfaceMaterials, *GeomCollection, TransformSelectionArr, InCollisionSampleSpacing, FTransform::Identity, bSetDefaultInternalMaterialsFromCollection, nullptr, IslandSplitSettings);

				UE::Private::FractureHelpers::ProcessNewlyFracturedBones(*GeomCollection, Index);

				int32 NewLen = Algo::RemoveIf(TransformSelectionArr, [&GeomCollection](int32 Bone)
				{
					return !GeomCollection->IsVisible(Bone); // remove already-fractured pieces from the to-cut list
				});
				TransformSelectionArr.SetNum(NewLen);

				if (ResultGeometryIndex == -1)
				{
					ResultGeometryIndex = Index;
				}
				if (Index > -1)
				{
					const int32 TransformIdx = GeomCollection->TransformIndex[Index];
					// after a successful cut, also consider any new bones added by the cut
					for (int32 NewBoneIdx = TransformIdx; NewBoneIdx < GeomCollection->NumElements(FGeometryCollection::TransformGroup); NewBoneIdx++)
					{
						TransformSelectionArr.Add(NewBoneIdx);
					}
				}
			};
			switch (PerCutMeshSelection)
			{
			case EMeshCutterPerCutMeshSelection::All:
				for (const UE::Geometry::FDynamicMesh3* Mesh : InDynCuttingMeshes)
				{
					ApplyCut(*Mesh);
				}
				break;
			case EMeshCutterPerCutMeshSelection::Random:
				ApplyCut(*InDynCuttingMeshes[RandStream.RandHelper(InDynCuttingMeshes.Num())]);
				break;
			case EMeshCutterPerCutMeshSelection::Sequential:
				ApplyCut(*InDynCuttingMeshes[SequentialMeshIndex]);
				SequentialMeshIndex = (SequentialMeshIndex + 1) % InDynCuttingMeshes.Num();
				break;
			default:
				checkNoEntry();
			}
		}

		if (ResultGeometryIndex > -1)
		{
			TArray<int32> ToRemove;
			for (int32 NewIdx = OriginalNumTransforms; NewIdx < GeomCollection->NumElements(FGeometryCollection::TransformGroup); ++NewIdx)
			{
				if (GeomCollection->IsRigid(NewIdx))
				{
					int32 ParentIdx = GeomCollection->Parent[NewIdx];
					if (ParentIdx >= OriginalNumTransforms)
					{
						do
						{
							ParentIdx = GeomCollection->Parent[ParentIdx];
						} while (GeomCollection->Parent[ParentIdx] >= OriginalNumTransforms);
						GeomCollection->ParentTransforms(ParentIdx, NewIdx);
					}
				}
				else
				{
					ToRemove.Add(NewIdx);
				}
			}
			FManagedArrayCollection::FProcessingParameters ProcessingParams;
			ProcessingParams.bDoValidation = false;
			GeomCollection->RemoveElements(FGeometryCollection::TransformGroup, ToRemove, ProcessingParams);
		}

		InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);

		return ResultGeometryIndex;
	}

	return INDEX_NONE;
}

int32 FFractureEngineFracturing::MeshCutter(TArray<FTransform>& MeshTransforms,
	FManagedArrayCollection& InOutCollection,
	const FDataflowTransformSelection& InTransformSelection,
	const UE::Geometry::FDynamicMesh3& InDynCuttingMesh,
	const int32 InRandomSeed,
	const float InChanceToFracture,
	const FIslandSplitSettings IslandSplitSettings,
	const float InCollisionSampleSpacing)
{
	const UE::Geometry::FDynamicMesh3* LocalCutterPtr = &InDynCuttingMesh;
	return MeshArrayCutter(MeshTransforms, InOutCollection, InTransformSelection, 
		TArrayView<const UE::Geometry::FDynamicMesh3*>(&LocalCutterPtr, 1), EMeshCutterPerCutMeshSelection::All,
		InRandomSeed, InChanceToFracture, IslandSplitSettings, InCollisionSampleSpacing);
}

int32 FFractureEngineFracturing::UniformFracture(
	FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection, 
	const FUniformFractureSettings& InUniformFractureSettings)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		RandomReduceSelection(InTransformSelection, InUniformFractureSettings.RandomSeed, InUniformFractureSettings.ChanceToFracture);
		UE::Private::FractureHelpers::ConvertToLeafSelection(*GeomCollection, InTransformSelection);
		TArray<int32> TransformSelectionArr = InTransformSelection.AsArrayValidated(*GeomCollection);

		if (!FFractureEngineSelection::IsBoneSelectionValid(InOutCollection, TransformSelectionArr))
		{
			return INDEX_NONE;
		}

		// Update global transforms and bounds		
		const TManagedArray<FTransform3f>& Transform = GeomCollection->GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& TransformToGeometryIndex = GeomCollection->GetAttribute<int32>(FGeometryCollection::TransformToGeometryIndexAttribute, FGeometryCollection::TransformGroup);
		const TManagedArray<FBox>& BoundingBoxes = GeomCollection->GetAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);

		TArray<FTransform> Transforms;
		GeometryCollectionAlgo::GlobalMatrices(Transform, GeomCollection->Parent, Transforms);

		int32 TransformCount = Transform.Num();
		TMap<int32, FBox> BoundsToBone;
		for (int32 Index = 0; Index < TransformCount; ++Index)
		{
			if (TransformToGeometryIndex[Index] > INDEX_NONE)
			{
				BoundsToBone.Add(Index, BoundingBoxes[TransformToGeometryIndex[Index]].TransformBy(Transforms[Index]));
			}
		}

		FBox BoundingBox(ForceInit);

		if (InUniformFractureSettings.GroupFracture)
		{
			FBox Bounds(ForceInit);
			for (int32 TransformIndex : TransformSelectionArr)
			{
				if (TransformToGeometryIndex[TransformIndex] > INDEX_NONE)
				{
					Bounds += BoundsToBone[TransformIndex];
				}
			}
			
			BoundingBox = Bounds;

			// Fracture
			FUniformFractureProcSettings UniformFractureProcSettings;
			UniformFractureProcSettings.BBox = BoundingBox;
			UniformFractureProcSettings.Transform = InUniformFractureSettings.Transform;
			UniformFractureProcSettings.MinVoronoiSites = InUniformFractureSettings.MinVoronoiSites;
			UniformFractureProcSettings.MaxVoronoiSites = InUniformFractureSettings.MaxVoronoiSites;
			UniformFractureProcSettings.InternalMaterialID = InUniformFractureSettings.InternalMaterialID;
			UniformFractureProcSettings.RandomSeed = InUniformFractureSettings.RandomSeed;
			UniformFractureProcSettings.SplitIslands = InUniformFractureSettings.SplitIslands;
			UniformFractureProcSettings.CloseVertexDistance = InUniformFractureSettings.CloseVertexDistance;
			UniformFractureProcSettings.VertexToSurfaceBridgeDistance = InUniformFractureSettings.VertexToSurfaceBridgeDistance;
			UniformFractureProcSettings.Grout = InUniformFractureSettings.Grout;
			UniformFractureProcSettings.NoiseSettings = InUniformFractureSettings.NoiseSettings;
			UniformFractureProcSettings.AddSamplesForCollision = InUniformFractureSettings.AddSamplesForCollision;
			UniformFractureProcSettings.CollisionSampleSpacing = InUniformFractureSettings.CollisionSampleSpacing;

			int ResultGeometryIndex = UE::Private::FractureHelpers::UniformFractureProc(
				GeomCollection,
				TransformSelectionArr,
				UniformFractureProcSettings);

			InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);

			return ResultGeometryIndex;
		}
		else
		{
			int ResultGeometryIndex = INDEX_NONE;

			for (int32 TransformIndex : TransformSelectionArr)
			{
				TArray<int32> TransformSelection;
				int32 Seed;

				if (TransformToGeometryIndex[TransformIndex] > INDEX_NONE)
				{
					TransformSelection.Add(TransformIndex);
					Seed = InUniformFractureSettings.RandomSeed + TransformIndex;
					BoundingBox = BoundsToBone[TransformIndex];

					// Fracture
					FUniformFractureProcSettings UniformFractureProcSettings;
					UniformFractureProcSettings.BBox = BoundingBox;
					UniformFractureProcSettings.Transform = InUniformFractureSettings.Transform;
					UniformFractureProcSettings.MinVoronoiSites = InUniformFractureSettings.MinVoronoiSites;
					UniformFractureProcSettings.MaxVoronoiSites = InUniformFractureSettings.MaxVoronoiSites;
					UniformFractureProcSettings.InternalMaterialID = InUniformFractureSettings.InternalMaterialID;
					UniformFractureProcSettings.RandomSeed = Seed;
					UniformFractureProcSettings.SplitIslands = InUniformFractureSettings.SplitIslands;
					UniformFractureProcSettings.CloseVertexDistance = InUniformFractureSettings.CloseVertexDistance;
					UniformFractureProcSettings.VertexToSurfaceBridgeDistance = InUniformFractureSettings.VertexToSurfaceBridgeDistance;
					UniformFractureProcSettings.Grout = InUniformFractureSettings.Grout;
					UniformFractureProcSettings.NoiseSettings = InUniformFractureSettings.NoiseSettings;
					UniformFractureProcSettings.AddSamplesForCollision = InUniformFractureSettings.AddSamplesForCollision;
					UniformFractureProcSettings.CollisionSampleSpacing = InUniformFractureSettings.CollisionSampleSpacing;

					ResultGeometryIndex = UE::Private::FractureHelpers::UniformFractureProc(
						GeomCollection,
						TransformSelection,
						UniformFractureProcSettings);
				}
			}

			InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);

			return ResultGeometryIndex;
		}
	}

	return INDEX_NONE;
}

namespace UE::Dataflow::Private
{
	static void SetBoneColor(FManagedArrayCollection& InCollection, const int32 InBoneIdx, const FLinearColor InBoneColor)
	{
		const TManagedArray<TSet<int32>>& Children = InCollection.GetAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FGeometryCollection::TransformGroup);
		TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);

		if (Children[InBoneIdx].Num() == 0)
		{
			BoneColors[InBoneIdx] = InBoneColor;
			return;
		}

		for (int32 Child : Children[InBoneIdx])
		{
			SetBoneColor(InCollection, Child, InBoneColor);
		}
	}

	static FLinearColor GetRandomColor(const FRandomStream& InRandomStream, const int32 InColorRangeMin, const int32 InColorRangeMax)
	{
		const uint8 R = static_cast<uint8>(InRandomStream.FRandRange(InColorRangeMin, InColorRangeMax));
		const uint8 G = static_cast<uint8>(InRandomStream.FRandRange(InColorRangeMin, InColorRangeMax));
		const uint8 B = static_cast<uint8>(InRandomStream.FRandRange(InColorRangeMin, InColorRangeMax));

		return FLinearColor(FColor(R, G, B, 255));
	}
}

void FFractureEngineFracturing::InitColors(FManagedArrayCollection& InCollection)
{
	const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

	TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);
	const int32 NumBones = BoneColors.Num();
	for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
	{
		BoneColors[BoneIdx] = DataflowSettings->TransformLevelColors.BlankColor;
	}

	TManagedArray<FLinearColor>& VertexColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup);
	const int32 NumVertices = VertexColors.Num();
	for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
	{
		VertexColors[VertexIdx] = DataflowSettings->TransformLevelColors.BlankColor;;
	}
}

void FFractureEngineFracturing::TransferBoneColorToVertexColor(FManagedArrayCollection& InCollection)
{
	const int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);

	TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& TransformToGeometryIndexArr = InCollection.GetAttribute<int32>(FGeometryCollection::TransformToGeometryIndexAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& VertexStartArr = InCollection.GetAttribute<int32>(FGeometryCollection::VertexStartAttribute, FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>& VertexCountArr = InCollection.GetAttribute<int32>(FGeometryCollection::VertexCountAttribute, FGeometryCollection::GeometryGroup);

	TManagedArray<FLinearColor>& VertexColorArr = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup);

	for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
	{
		const int32 GeometryIndex = TransformToGeometryIndexArr[TransformIdx];
		// Only transfer color to non clusters
		if (GeometryIndex != -1)
		{
			const int32 VertexStart = VertexStartArr[GeometryIndex];
			const int32 VertexCount = VertexCountArr[GeometryIndex];

			for (int32 VertexIdx = VertexStart; VertexIdx < VertexStart + VertexCount; ++VertexIdx)
			{
				VertexColorArr[VertexIdx] = BoneColors[TransformIdx];
			}
		}
	}
}

void FFractureEngineFracturing::SetBoneColorByParent(
	FManagedArrayCollection& InCollection,
	const FRandomStream& InRandomStream,
	int32 InLevel,
	const int32 InColorRangeMin,
	const int32 InColorRangeMax)
{
	TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);
	const int32 NumBones = BoneColors.Num();
	const TManagedArray<int32>& Levels = InCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<TSet<int32>>& Children = InCollection.GetAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FGeometryCollection::TransformGroup);

	if (InLevel == 0)
	{
		FLinearColor Color = UE::Dataflow::Private::GetRandomColor(InRandomStream, InColorRangeMin, InColorRangeMax);

		for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
		{
			BoneColors[BoneIdx] = Color;
		}
	}
	else if (InLevel > 0)
	{
		for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
		{
			if (Levels[BoneIdx] == InLevel - 1)
			{
				FLinearColor Color = UE::Dataflow::Private::GetRandomColor(InRandomStream, InColorRangeMin, InColorRangeMax);

				for (int32 ChildBoneIdx : Children[BoneIdx])
				{
					UE::Dataflow::Private::SetBoneColor(InCollection, ChildBoneIdx, Color);
				}
			}
		}
	}
}

void FFractureEngineFracturing::SetBoneColorByLevel(FManagedArrayCollection& InCollection, int32 InLevel)
{
	const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();
	const int32 NumTransformLevelColors = DataflowSettings->TransformLevelColors.LevelColors.Num();

	TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);
	const int32 NumBones = BoneColors.Num();
	const TManagedArray<int32>& Levels = InCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup);

	for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
	{
		if (InLevel >= 0)
		{
			if (Levels[BoneIdx] >= InLevel)
			{
				BoneColors[BoneIdx] = DataflowSettings->TransformLevelColors.LevelColors[InLevel % NumTransformLevelColors];
			}
			else
			{
				BoneColors[BoneIdx] = DataflowSettings->TransformLevelColors.BlankColor;
			}
		}
	}
}

 void FFractureEngineFracturing::SetBoneColorByCluster(
	 FManagedArrayCollection& InCollection, 
	 const FRandomStream& InRandomStream, 
	 int32 InLevel,
	 const int32 InColorRangeMin,
	 const int32 InColorRangeMax)
{
	 const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

	TArray<FLinearColor> RandomColors;

	FRandomStream Random(1);
	for (int i = 0; i < 100; i++)
	{
		const uint8 R = static_cast<uint8>(Random.FRandRange(InColorRangeMin, InColorRangeMax));
		const uint8 G = static_cast<uint8>(Random.FRandRange(InColorRangeMin, InColorRangeMax));
		const uint8 B = static_cast<uint8>(Random.FRandRange(InColorRangeMin, InColorRangeMax));
		RandomColors.Push(FLinearColor(FColor(R, G, B, 255)));
	}

	const TManagedArray<int32>& Parents = InCollection.GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

	const TManagedArray<int32>& Levels = InCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup);
	TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);

	const int32 NumParents = Parents.Num();

	for (int32 BoneIndex = 0, NumBones = NumParents; BoneIndex < NumBones; ++BoneIndex)
	{
		FLinearColor BoneColor = FLinearColor(FColor::Black);

		if (Levels[BoneIndex] >= InLevel)
		{
			// go up until we find parent at the required ViewLevel
			int32 Bone = BoneIndex;
			while (Bone != -1 && Levels[Bone] > InLevel)
			{
				Bone = Parents[Bone];
			}

			int32 ColorIndex = Bone + 1; // parent can be -1 for root, range [-1..n]
			BoneColor = RandomColors[ColorIndex % RandomColors.Num()];

			BoneColor = BoneColor.LinearRGBToHSV();
			BoneColor.B *= .5;
			BoneColor = BoneColor.HSVToLinearRGB();
		}
		else
		{
			BoneColor = DataflowSettings->TransformLevelColors.BlankColor;;
		}

		BoneColors[BoneIndex] = BoneColor;
	}
}

 void FFractureEngineFracturing::SetBoneColorByLeafLevel(FManagedArrayCollection& InCollection, int32 InLevel)
 {
	 const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();
	 const int32 NumTransformLevelColors = DataflowSettings->TransformLevelColors.LevelColors.Num();

	 TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);
	 const int32 NumBones = BoneColors.Num();
	 const TManagedArray<int32>& Levels = InCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup);
	 const TManagedArray<TSet<int32>>& Children = InCollection.GetAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FGeometryCollection::TransformGroup);

	 for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
	 {
		 if (Levels[BoneIdx] >= InLevel)
		 {
			 BoneColors[BoneIdx] = Children[BoneIdx].Num() == 0 ? DataflowSettings->TransformLevelColors.LevelColors[Levels[BoneIdx] % NumTransformLevelColors] : FColor::Black;
		 }
		 else
		 {
			 BoneColors[BoneIdx] = DataflowSettings->TransformLevelColors.BlankColor;
		 }
	 }
 }

 void FFractureEngineFracturing::SetBoneColorByLeaf(FManagedArrayCollection& InCollection, 
	 const FRandomStream& InRandomStream, 
	 int32 InLevel, 
	 const int32 InColorRangeMin,
	 const int32 InColorRangeMax)
 {
	 const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

	 TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);
	 const TManagedArray<int32>& Levels = InCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup);
	 const int32 NumBones = BoneColors.Num();

	 for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
	 {
		 if (Levels[BoneIdx] >= InLevel)
		 {
			 BoneColors[BoneIdx] = UE::Dataflow::Private::GetRandomColor(InRandomStream, InColorRangeMin, InColorRangeMax);
		 }
		 else
		 {
			 BoneColors[BoneIdx] = DataflowSettings->TransformLevelColors.BlankColor;
		 }
	 }
 }

 void FFractureEngineFracturing::SetBoneColorByAttr(
	 FManagedArrayCollection& InCollection,
	 const FString InAttribute,
	 const float InMinAttrValue,
	 float InMaxAttrValue,
	 const FLinearColor InMinColor,
	 const FLinearColor InMaxColor)
 {
	 const FName AttrName = FName(*InAttribute);
	 if (InCollection.HasAttribute(AttrName, FGeometryCollection::TransformGroup))
	 {
		 const TManagedArray<float>& AttrValues = InCollection.GetAttribute<float>(AttrName, FGeometryCollection::TransformGroup);
		 TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);

		 for (int32 Idx = 0; Idx < AttrValues.Num(); ++Idx)
		 {
			 float AttrValue = AttrValues[Idx];

			 // Make sure AttrValue is valid
			 AttrValue = FMath::Max(AttrValue, InMinAttrValue);
			 AttrValue = FMath::Min(AttrValue, InMaxAttrValue);
			 if (InMaxAttrValue < InMinAttrValue)
			 {
				 InMaxAttrValue = InMinAttrValue + 0.01f;
			 }

			 // Lerp between InMinColor and InMaxColor
			 float Alpha = (AttrValue - InMinAttrValue) / (InMaxAttrValue - InMinAttrValue);
			 FLinearColor NewColor = FLinearColor::LerpUsingHSV(InMinColor, InMaxColor, Alpha);
			 BoneColors[Idx] = NewColor;
		 }
	 }
 }

 void FFractureEngineFracturing::SetBoneColorRandom(
	 FManagedArrayCollection& InCollection, 
	 const FRandomStream& InRandomStream)
 {
	 TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::BoneColorAttribute, FGeometryCollection::TransformGroup);
	 const int32 NumBones = BoneColors.Num();

	 for (int32 Idx = 0; Idx < NumBones; ++Idx)
	 {
		 const uint8 R = static_cast<uint8>(InRandomStream.FRandRange(5, 105));
		 const uint8 G = static_cast<uint8>(InRandomStream.FRandRange(5, 105));
		 const uint8 B = static_cast<uint8>(InRandomStream.FRandRange(5, 105));

		 BoneColors[Idx] = FLinearColor(FColor(R, G, B, 255));
	 }
 }
