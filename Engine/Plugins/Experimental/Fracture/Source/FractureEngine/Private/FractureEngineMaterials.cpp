// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineMaterials.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"

namespace UE::FractureEngineMaterialHelpers
{
	void SetMaterialOnFaceRange(const TManagedArray<bool>& Internal, TManagedArray<int32>& Material, int32 Start, int32 End, FFractureEngineMaterials::ETargetFaces TargetFaces, int32 MaterialID)
	{
		bool bAllFaces = TargetFaces == FFractureEngineMaterials::ETargetFaces::AllFaces;
		bool bInternalFaces = TargetFaces == FFractureEngineMaterials::ETargetFaces::InternalFaces;
		for (int32 FIdx = Start; FIdx < End; ++FIdx)
		{
			if (bAllFaces || bInternalFaces == Internal[FIdx])
			{
				Material[FIdx] = MaterialID;
			}
		}
	}
}
 

void FFractureEngineMaterials::SetMaterial(FManagedArrayCollection& InCollection, const TArray<int32>& InBoneSelection, ETargetFaces TargetFaces, int32 MaterialID)
{
	GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);

	if (!MeshFacade.IsValid())
	{
		return;
	}

	const TManagedArray<int32>& TransformToGeometryIndex = MeshFacade.TransformToGeometryIndexAttribute.Get();
	const TManagedArray<int32>& FaceStart = MeshFacade.FaceStartAttribute.Get();
	const TManagedArray<int32>& FaceCount = MeshFacade.FaceCountAttribute.Get();
	const TManagedArray<bool>& Internal = MeshFacade.InternalAttribute.Get();
	TManagedArray<int32>& Material = MeshFacade.MaterialIDAttribute.Modify();

	GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
	TArray<int32> BoneIndices = InBoneSelection;
	TransformSelectionFacade.ConvertSelectionToRigidNodes(BoneIndices);
	TransformSelectionFacade.Sanitize(BoneIndices);

	for (int32 Index : BoneIndices)
	{
		if (TransformToGeometryIndex[Index] > INDEX_NONE)
		{
			int32 Start = FaceStart[TransformToGeometryIndex[Index]];
			int32 End = Start + FaceCount[TransformToGeometryIndex[Index]];
			
			UE::FractureEngineMaterialHelpers::SetMaterialOnFaceRange(Internal, Material, Start, End, TargetFaces, MaterialID);
		}
	}
}


void FFractureEngineMaterials::SetMaterialOnGeometryAfter(FManagedArrayCollection& InCollection, int32 FirstGeometryIndex, ETargetFaces TargetFaces, int32 MaterialID)
{
	GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);

	if (!MeshFacade.IsValid())
	{
		return;
	}

	const TManagedArray<int32>& TransformToGeometryIndex = MeshFacade.TransformToGeometryIndexAttribute.Get();
	const TManagedArray<int32>& FaceStart = MeshFacade.FaceStartAttribute.Get();
	const TManagedArray<int32>& FaceCount = MeshFacade.FaceCountAttribute.Get();
	const TManagedArray<bool>& Internal = MeshFacade.InternalAttribute.Get();
	TManagedArray<int32>& Material = MeshFacade.MaterialIDAttribute.Modify();

	for (int32 GeoIdx = FirstGeometryIndex, NumGeometry = FaceStart.Num(); GeoIdx < NumGeometry; ++GeoIdx)
	{
		int32 Start = FaceStart[GeoIdx];
		int32 End = Start + FaceCount[GeoIdx];

		UE::FractureEngineMaterialHelpers::SetMaterialOnFaceRange(Internal, Material, Start, End, TargetFaces, MaterialID);
	}
}


