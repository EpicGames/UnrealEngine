// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEnginePlugin.h"




class FFractureEnginePlugin : public IFractureEnginePlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FFractureEnginePlugin, FractureEngine )



void FFractureEnginePlugin::StartupModule()
{

}


void FFractureEnginePlugin::ShutdownModule()
{

}



