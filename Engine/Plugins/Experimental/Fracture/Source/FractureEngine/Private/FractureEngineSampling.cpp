// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineSampling.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "FractureEngineSelection.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "Algo/RemoveIf.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "Sampling/MeshSurfacePointSampling.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureEngineSampling)

void FFractureEngineSampling::ComputeUniformPointSampling(const UE::Geometry::FDynamicMesh3& InMesh,
	const float InSamplingRadius,
	const int32 InMaxNumSamples,
	const float InSubSampleDensity,
	const int32 InRandomSeed,
	TArray<FTransform>& OutSamples,
	TArray<int32>& OutTriangleIDs,
	TArray<FVector>& OutBarycentricCoords)
{
	if (InMesh.VertexCount())
	{
		UE::Geometry::FMeshSurfacePointSampling Sampler;
		Sampler.SampleRadius = InSamplingRadius;
		Sampler.MaxSamples = InMaxNumSamples;
		Sampler.RandomSeed = InRandomSeed;
		Sampler.SubSampleDensity = InSubSampleDensity;
		Sampler.bComputeBarycentrics = true;

		Sampler.ComputePoissonSampling(InMesh);

		int32 NumSamples = Sampler.Samples.Num();
		OutSamples.SetNumUninitialized(NumSamples);
		OutTriangleIDs.SetNumUninitialized(NumSamples);
		OutBarycentricCoords.SetNumUninitialized(NumSamples);

		for (int32 Idx = 0; Idx < NumSamples; ++Idx)
		{
			const UE::Geometry::FFrame3d& Frame = Sampler.Samples[Idx];
			OutSamples[Idx] = Frame.ToFTransform();
			OutTriangleIDs[Idx] = Sampler.TriangleIDs[Idx];
			OutBarycentricCoords[Idx] = Sampler.BarycentricCoords[Idx];
		}
	}
}

void FFractureEngineSampling::ComputeNonUniformPointSampling(const UE::Geometry::FDynamicMesh3& InMesh,
	const float InSamplingRadius,
	const int32 InMaxNumSamples,
	const float InSubSampleDensity,
	const int32 InRandomSeed,
	const float InMaxSamplingRadius,
	const ENonUniformSamplingDistributionMode InSizeDistribution,
	const float InSizeDistributionPower,
	TArray<FTransform>& OutSamples,
	TArray<float>& OutSampleRadii,
	TArray<int32>& OutTriangleIDs,
	TArray<FVector>& OutBarycentricCoords)
{
	if (InMesh.VertexCount())
	{
		UE::Geometry::FMeshSurfacePointSampling Sampler;
		Sampler.SampleRadius = InSamplingRadius;
		Sampler.MaxSamples = InMaxNumSamples;
		Sampler.RandomSeed = InRandomSeed;
		Sampler.SubSampleDensity = InSubSampleDensity;
		Sampler.bComputeBarycentrics = true;

		if (InMaxSamplingRadius > InSamplingRadius)
		{
			Sampler.MaxSampleRadius = InMaxSamplingRadius;
			Sampler.SizeDistribution = (UE::Geometry::FMeshSurfacePointSampling::ESizeDistribution)(int)(InSizeDistribution);
			Sampler.SizeDistributionPower = FMath::Clamp(InSizeDistributionPower, 1.f, 10.f);
		}

		Sampler.ComputePoissonSampling(InMesh);

		int32 NumSamples = Sampler.Samples.Num();
		OutSamples.SetNumUninitialized(NumSamples);
		OutSampleRadii.SetNumUninitialized(NumSamples);
		OutTriangleIDs.SetNumUninitialized(NumSamples);
		OutBarycentricCoords.SetNumUninitialized(NumSamples);

		for (int32 Idx = 0; Idx < NumSamples; ++Idx)
		{
			const UE::Geometry::FFrame3d& Frame = Sampler.Samples[Idx];
			OutSamples[Idx] = Frame.ToFTransform();
			OutSampleRadii[Idx] = Sampler.Radii[Idx];
			OutTriangleIDs[Idx] = Sampler.TriangleIDs[Idx];
			OutBarycentricCoords[Idx] = Sampler.BarycentricCoords[Idx];
		}
	}
}

void FFractureEngineSampling::ComputeVertexWeightedPointSampling(const UE::Geometry::FDynamicMesh3& InMesh,
	const TArray<float>& InVertexWeights,
	const float InSamplingRadius,
	const int32 InMaxNumSamples,
	const float InSubSampleDensity,
	const int32 InRandomSeed,
	const float InMaxSamplingRadius,
	const ENonUniformSamplingDistributionMode InSizeDistribution,
	const float InSizeDistributionPower,
	const ENonUniformSamplingWeightMode InWeightMode,
	const bool InInvertWeights,
	TArray<FTransform>& OutSamples,
	TArray<float>& OutSampleRadii,
	TArray<int32>& OutTriangleIDs,
	TArray<FVector>& OutBarycentricCoords)
{
	if (InMesh.VertexCount())
	{
		UE::Geometry::FMeshSurfacePointSampling Sampler;
		Sampler.SampleRadius = InSamplingRadius;
		Sampler.MaxSamples = InMaxNumSamples;
		Sampler.RandomSeed = InRandomSeed;
		Sampler.SubSampleDensity = InSubSampleDensity;
		Sampler.bComputeBarycentrics = true;

		bool bIsNonUniform = false;

		if (InMaxSamplingRadius > InSamplingRadius)
		{
			bIsNonUniform = true;

			Sampler.MaxSampleRadius = InMaxSamplingRadius;
			Sampler.SizeDistribution = (UE::Geometry::FMeshSurfacePointSampling::ESizeDistribution)(int)(InSizeDistribution);
			Sampler.SizeDistributionPower = FMath::Clamp(InSizeDistributionPower, 1.f, 10.f);
		}

		const int32 NumVertices = InMesh.VertexCount();

		if (bIsNonUniform && InVertexWeights.Num() == NumVertices)
		{
			TArray<double> WeightArr; WeightArr.SetNumUninitialized(NumVertices);
			for (int32 Idx = 0; Idx < NumVertices; ++Idx) {	WeightArr[Idx] = (double)InVertexWeights[Idx]; }

			Sampler.VertexWeights = WeightArr;
			Sampler.bUseVertexWeights = true;
			Sampler.InterpretWeightMode = (UE::Geometry::FMeshSurfacePointSampling::EInterpretWeightMode)(int)(InWeightMode);
			Sampler.bInvertWeights = InInvertWeights;
		}

		Sampler.ComputePoissonSampling(InMesh);

		int32 NumSamples = Sampler.Samples.Num();
		OutSamples.SetNumUninitialized(NumSamples);
		OutSampleRadii.SetNumUninitialized(NumSamples);
		OutTriangleIDs.SetNumUninitialized(NumSamples);
		OutBarycentricCoords.SetNumUninitialized(NumSamples);

		for (int32 Idx = 0; Idx < NumSamples; ++Idx)
		{
			const UE::Geometry::FFrame3d& Frame = Sampler.Samples[Idx];
			OutSamples[Idx] = Frame.ToFTransform();
			OutSampleRadii[Idx] = Sampler.Radii[Idx];
			OutTriangleIDs[Idx] = Sampler.TriangleIDs[Idx];
			OutBarycentricCoords[Idx] = Sampler.BarycentricCoords[Idx];
		}
	}
}

