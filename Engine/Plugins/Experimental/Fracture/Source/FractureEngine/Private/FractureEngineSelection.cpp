// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineSelection.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "Algo/RemoveIf.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/TransformCollection.h"

static void ConvertSelectionSetToSelectionArr(const TSet<int32>& SelectionSet, TArray<int32>& SelectionArr)
{
	SelectionArr.Empty(SelectionSet.Num());

	for (auto& BoneIdx : SelectionSet)
	{
		SelectionArr.Add(BoneIdx);
	}
}


bool FFractureEngineSelection::IsBoneSelectionValid(const FManagedArrayCollection& Collection, const TArray<int32>& SelectedBones)
{
	return IsSelectionValid(Collection, SelectedBones, FGeometryCollection::TransformGroup);
}

bool FFractureEngineSelection::IsSelectionValid(const FManagedArrayCollection& Collection, const TArray<int32>& SelectedItems, const FName ItemGroup)
{
	int32 NumItems = Collection.NumElements(ItemGroup);
	for (int32 Idx : SelectedItems)
	{
		if (Idx < 0 || Idx >= NumItems)
		{
			return false;
		}
	}
	return true;
}

void FFractureEngineSelection::GetRootBones(const FManagedArrayCollection& Collection, TArray<int32>& RootBonesOut)
{
	if (Collection.HasGroup(FGeometryCollection::TransformGroup) &&
		Collection.HasAttribute("Parent", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

		for (int32 Idx = 0; Idx < Parents.Num(); ++Idx)
		{
			if (Parents[Idx] == FGeometryCollection::Invalid)
			{
				RootBonesOut.Add(Idx);
			}
		}
	}
}

void FFractureEngineSelection::GetRootBones(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection)
{
	if (Collection.HasGroup(FGeometryCollection::TransformGroup) &&
		Collection.HasAttribute("Parent", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

		for (int32 Idx = 0; Idx < Parents.Num(); ++Idx)
		{
			if (Parents[Idx] == FGeometryCollection::Invalid)
			{
				TransformSelection.SetSelected(Idx);
			}
		}
	}
}

void FFractureEngineSelection::SelectParent(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones)
{
	if (Collection.HasGroup(FGeometryCollection::TransformGroup) &&
		Collection.HasAttribute("Parent", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

		TSet<int32> NewSelection;
		for (int32 Bone : SelectedBones)
		{
			int32 ParentBone = Parents[Bone];
			if (ParentBone != FGeometryCollection::Invalid)
			{
				NewSelection.Add(ParentBone);
			}
		}

		ConvertSelectionSetToSelectionArr(NewSelection, SelectedBones);
	}
}

void FFractureEngineSelection::SelectParent(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection)
{
	TArray<int32> SelectionArr;
	TransformSelection.AsArray(SelectionArr);

	SelectParent(Collection, SelectionArr);

	TransformSelection.SetFromArray(SelectionArr);
}

void FFractureEngineSelection::SelectChildren(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones)
{
	if (Collection.HasGroup(FGeometryCollection::TransformGroup) &&
		Collection.HasAttribute("Children", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<TSet<int32>>& Children = Collection.GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);

		TSet<int32> NewSelection;
		for (int32 Bone : SelectedBones)
		{
			if (Children[Bone].IsEmpty())
			{
				NewSelection.Add(Bone);
				continue;
			}
			for (int32 Child : Children[Bone])
			{
				NewSelection.Add(Child);
			}
		}

		ConvertSelectionSetToSelectionArr(NewSelection, SelectedBones);
	}
}

void FFractureEngineSelection::SelectChildren(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection)
{
	TArray<int32> SelectionArr;
	TransformSelection.AsArray(SelectionArr);

	SelectChildren(Collection, SelectionArr);

	TransformSelection.SetFromArray(SelectionArr);
}

void FFractureEngineSelection::SelectSiblings(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones)
{
	if (Collection.HasGroup(FGeometryCollection::TransformGroup) &&
		Collection.HasAttribute("Parent", FGeometryCollection::TransformGroup) &&
		Collection.HasAttribute("Children", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
		const TManagedArray<TSet<int32>>& Children = Collection.GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);

		TSet<int32> NewSelection;
		for (int32 Bone : SelectedBones)
		{
			int32 ParentBone = Parents[Bone];
			if (ParentBone != FGeometryCollection::Invalid)
			{
				for (int32 Child : Children[ParentBone])
				{
					NewSelection.Add(Child);
				}
			}
		}

		ConvertSelectionSetToSelectionArr(NewSelection, SelectedBones);
	}
}

void FFractureEngineSelection::SelectSiblings(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection)
{
	TArray<int32> SelectionArr;
	TransformSelection.AsArray(SelectionArr);

	SelectSiblings(Collection, SelectionArr);

	TransformSelection.SetFromArray(SelectionArr);
}

void FFractureEngineSelection::SelectLevel(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones)
{
	if (Collection.HasGroup(FGeometryCollection::TransformGroup) &&
		Collection.HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<int32>& Levels = Collection.GetAttribute<int32>("Levels", FGeometryCollection::TransformGroup);

		TSet<int32> NewSelection;
		for (int32 Bone : SelectedBones)
		{
			int32 Level = Levels[Bone];
			for (int32 TransformIdx = 0; TransformIdx < Collection.NumElements(FTransformCollection::TransformGroup); ++TransformIdx)
			{
				if (Levels[TransformIdx] == Level)
				{
					NewSelection.Add(TransformIdx);
				}
			}
		}

		ConvertSelectionSetToSelectionArr(NewSelection, SelectedBones);
	}
}

void FFractureEngineSelection::SelectLevel(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection)
{
	TArray<int32> SelectionArr;
	TransformSelection.AsArray(SelectionArr);

	SelectLevel(Collection, SelectionArr);

	TransformSelection.SetFromArray(SelectionArr);
}

void FFractureEngineSelection::SelectContact(FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones)
{
	if (GeometryCollection.HasGroup(FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup) &&
		GeometryCollection.HasAttribute("TransformToGeometryIndex", FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasGroup(FGeometryCollection::GeometryGroup) &&
		GeometryCollection.HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
	{
		FGeometryCollectionProximityUtility ProximityUtility(&GeometryCollection);
		ProximityUtility.RequireProximity();

		const TManagedArray<int32>& TransformIndex = GeometryCollection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollection.GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
		const TManagedArray<TSet<int32>>& Proximity = GeometryCollection.GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		TSet<int32> NewSelection;
		for (int32 Bone : SelectedBones)
		{
			NewSelection.Add(Bone);
			int32 GeometryIdx = TransformToGeometryIndex[Bone];
			if (GeometryIdx != INDEX_NONE)
			{
				const TSet<int32>& Neighbors = Proximity[GeometryIdx];
				for (int32 NeighborGeometryIndex : Neighbors)
				{
					NewSelection.Add(TransformIndex[NeighborGeometryIndex]);
				}
			}
		}

		ConvertSelectionSetToSelectionArr(NewSelection, SelectedBones);
	}
}

void FFractureEngineSelection::SelectContact(FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection)
{
	TArray<int32> SelectionArr;
	TransformSelection.AsArray(SelectionArr);

	SelectContact(GeometryCollection, SelectionArr);

	TransformSelection.SetFromArray(SelectionArr);
}

void FFractureEngineSelection::SelectLeaf(const FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones)
{
	if (GeometryCollection.HasGroup(FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasAttribute("Level", FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasAttribute("SimulationType", FGeometryCollection::TransformGroup))
	{
		SelectedBones.Empty();

		int32 ViewLevel = -1;

		FGeometryCollectionClusteringUtility::GetBonesToLevel(&GeometryCollection, ViewLevel, SelectedBones, true, true);

		const TManagedArray<int32>& SimType = GeometryCollection.GetAttribute<int32>("SimulationType", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>* Levels = GeometryCollection.FindAttributeTyped<int32>("Level", FGeometryCollection::TransformGroup);
		
		SelectedBones.SetNum(Algo::RemoveIf(SelectedBones, [&](int32 BoneIdx)
			{
				return SimType[BoneIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid
					|| (ViewLevel != -1 && Levels && (*Levels)[BoneIdx] != ViewLevel);
			}));
	}
}

void FFractureEngineSelection::SelectLeaf(const FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection)
{
	TArray<int32> SelectionArr;

	SelectLeaf(GeometryCollection, SelectionArr);

	TransformSelection.SetFromArray(SelectionArr);
}

void FFractureEngineSelection::SelectCluster(const FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones)
{
	if (GeometryCollection.HasGroup(FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasAttribute("Level", FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasAttribute("SimulationType", FGeometryCollection::TransformGroup))
	{
		SelectedBones.Empty();

		int32 ViewLevel = -1;

		FGeometryCollectionClusteringUtility::GetBonesToLevel(&GeometryCollection, ViewLevel, SelectedBones, true, true);

		const TManagedArray<int32>& SimType = GeometryCollection.GetAttribute<int32>("SimulationType", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>* Levels = GeometryCollection.FindAttributeTyped<int32>("Level", FGeometryCollection::TransformGroup);

		SelectedBones.SetNum(Algo::RemoveIf(SelectedBones, [&](int32 BoneIdx)
			{
				return SimType[BoneIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid
					|| (ViewLevel != -1 && Levels && (*Levels)[BoneIdx] != ViewLevel);
			}));
	}
}

void FFractureEngineSelection::SelectCluster(const FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection)
{
	TArray<int32> SelectionArr;

	SelectCluster(GeometryCollection, SelectionArr);

	TransformSelection.SetFromArray(SelectionArr);
}

static void RandomShuffleArray(TArray<int32>& InArray, const bool Deterministic, const float RandomSeed)
{
	FRandomStream Stream(RandomSeed);

	int32 LastIndex = InArray.Num() - 1;

	static const int32 NumIterationsMin = 10;
	static const int32 NumIterationsMax = 50;

	int32 NumIterations = Deterministic ? Stream.RandRange(NumIterationsMin, NumIterationsMax) : FMath::RandRange(NumIterationsMin, NumIterationsMax);

	for (int32 IterIdx = 0; IterIdx < NumIterations; ++IterIdx)
	{
		for (int32 Idx = 0; Idx <= LastIndex; ++Idx)
		{
			int32 Index;
			if (Deterministic)
			{
				Index = Stream.RandRange(Idx, LastIndex);
			}
			else
			{
				Index = FMath::RandRange(Idx, LastIndex);
			}

			if (Idx != Index)
			{
				InArray.Swap(Idx, Index);
			}
		}
	}
}

void FFractureEngineSelection::SelectByPercentage(TArray<int32>& SelectedBones, const int32 Percentage, const bool Deterministic, const float RandomSeed)
{
	RandomShuffleArray(SelectedBones, Deterministic, RandomSeed);

	int32 NewNumElements = FMath::RoundToInt32(SelectedBones.Num() * Percentage * 0.01f);
	SelectedBones.SetNum(NewNumElements);
}

void FFractureEngineSelection::SelectByPercentage(FDataflowTransformSelection& TransformSelection, const int32 Percentage, const bool Deterministic, const float RandomSeed)
{
	TArray<int32> SelectionArr;
	TransformSelection.AsArray(SelectionArr);

	SelectByPercentage(SelectionArr, Percentage, Deterministic, RandomSeed);

	TransformSelection.SetFromArray(SelectionArr);
}


void FFractureEngineSelection::SelectBySize(FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones, const float SizeMin, const float SizeMax)
{
	if (GeometryCollection.HasGroup(FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasAttribute("Size", FGeometryCollection::TransformGroup))
	{
		SelectedBones.Empty();

		FGeometryCollectionConvexUtility::SetVolumeAttributes(&GeometryCollection);

		const TManagedArray<float>& Sizes = GeometryCollection.GetAttribute<float>("Size", FTransformCollection::TransformGroup);

		for (int32 BoneIdx = 0; BoneIdx < Sizes.Num(); ++BoneIdx)
		{
			if (Sizes[BoneIdx] >= SizeMin && Sizes[BoneIdx] <= SizeMax)
			{
				SelectedBones.Add(BoneIdx);
			}
		}
	}
}

void FFractureEngineSelection::SelectBySize(FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection, const float SizeMin, const float SizeMax)
{
	TArray<int32> SelectionArr;

	SelectBySize(GeometryCollection, SelectionArr, SizeMin, SizeMax);

	TransformSelection.SetFromArray(SelectionArr);
}

void FFractureEngineSelection::SelectByVolume(FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones, const float VolumeMin, const float VolumeMax)
{
	if (GeometryCollection.HasGroup(FGeometryCollection::TransformGroup) &&
		GeometryCollection.HasAttribute("Volume", FGeometryCollection::TransformGroup))
	{
		SelectedBones.Empty();

		FGeometryCollectionConvexUtility::SetVolumeAttributes(&GeometryCollection);

		const TManagedArray<float>& Volumes = GeometryCollection.GetAttribute<float>("Volume", FTransformCollection::TransformGroup);

		for (int32 BoneIdx = 0; BoneIdx < Volumes.Num(); ++BoneIdx)
		{
			if (Volumes[BoneIdx] >= VolumeMin && Volumes[BoneIdx] <= VolumeMax)
			{
				SelectedBones.Add(BoneIdx);
			}
		}
	}
}

void FFractureEngineSelection::SelectByVolume(FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection, const float VolumeMin, const float VolumeMax)
{
	TArray<int32> SelectionArr;

	SelectByVolume(GeometryCollection, SelectionArr, VolumeMin, VolumeMax);

	TransformSelection.SetFromArray(SelectionArr);
}

