// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEngineUtility.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "FractureEngineSelection.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "Algo/RemoveIf.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureEngineUtility)

void FFractureEngineUtility::ConvertBoxToVertexAndTriangleData(const FBox& InBox, TArray<FVector3f>& OutVertices, TArray<FIntVector>& OutTriangles)
{
	const int32 NumVertices = 8;
	const int32 NumTriangles = 12;

	OutVertices.AddUninitialized(NumVertices);
	OutTriangles.AddUninitialized(NumTriangles);

	FVector Min = InBox.Min;
	FVector Max = InBox.Max;

	// Add vertices
	OutVertices[0] = FVector3f(Min);
	OutVertices[1] = FVector3f(Max.X, Min.Y, Min.Z);
	OutVertices[2] = FVector3f(Max.X, Max.Y, Min.Z);
	OutVertices[3] = FVector3f(Min.X, Max.Y, Min.Z);
	OutVertices[4] = FVector3f(Min.X, Min.Y, Max.Z);
	OutVertices[5] = FVector3f(Max.X, Min.Y, Max.Z);
	OutVertices[6] = FVector3f(Max);
	OutVertices[7] = FVector3f(Min.X, Max.Y, Max.Z);

	// Add triangles
	OutTriangles[0] = FIntVector(0, 1, 3); OutTriangles[1] = FIntVector(1, 2, 3);
	OutTriangles[2] = FIntVector(0, 4, 1); OutTriangles[3] = FIntVector(4, 5, 1);
	OutTriangles[4] = FIntVector(5, 2, 1); OutTriangles[5] = FIntVector(5, 6, 2);
	OutTriangles[6] = FIntVector(3, 2, 6); OutTriangles[7] = FIntVector(7, 3, 6);
	OutTriangles[8] = FIntVector(0, 3, 7); OutTriangles[9] = FIntVector(4, 0, 7);
	OutTriangles[10] = FIntVector(5, 4, 7); OutTriangles[11] = FIntVector(5, 7, 6);
}


void FFractureEngineUtility::ConstructMesh(UE::Geometry::FDynamicMesh3& OutMesh, const TArray<FVector3f>& InVertices, const TArray<FIntVector>& InTriangles)
{
	for (int32 VertexIdx = 0; VertexIdx < InVertices.Num(); ++VertexIdx)
	{
		OutMesh.AppendVertex(FVector(InVertices[VertexIdx]));
	}

	int GroupID = 0;
	for (int32 TriangleIdx = 0; TriangleIdx < InTriangles.Num(); ++TriangleIdx)
	{
		OutMesh.AppendTriangle(InTriangles[TriangleIdx].X, InTriangles[TriangleIdx].Y, InTriangles[TriangleIdx].Z, GroupID);
	}
}


void FFractureEngineUtility::DeconstructMesh(const UE::Geometry::FDynamicMesh3& InMesh, TArray<FVector3f>& OutVertices, TArray<FIntVector>& OutTriangles)
{
	const int32 NumVertices = InMesh.VertexCount();
	const int32 NumTriangles = InMesh.TriangleCount();

	if (NumVertices > 0 && NumTriangles > 0)
	{
		// This will contain the valid triangles only
		OutTriangles.Reserve(InMesh.TriangleCount());

		// DynamicMesh.TrianglesItr() returns the valid triangles only
		for (UE::Geometry::FIndex3i Tri : InMesh.TrianglesItr())
		{
			OutTriangles.Add(FIntVector(Tri.A, Tri.B, Tri.C));
		}

		// This will contain all the vertices (invalid ones too)
		// Otherwise the IDs need to be remaped
		OutVertices.AddZeroed(InMesh.MaxVertexID());

		// DynamicMesh.VertexIndicesItr() returns the valid vertices only
		for (int32 VertexID : InMesh.VertexIndicesItr())
		{
			OutVertices[VertexID] = (FVector3f)InMesh.GetVertex(VertexID);
		}
	}
}

namespace UE::Dataflow::Private
{
	static constexpr double VolDimScale = .01; // compute volumes in meters instead of cm, for saner units at typical scales
}

static double GetTotalVolume(const FGeometryCollection& Collection, const TArray<double>& Volumes)
{
	double Sum = 0.0;
	if (Volumes.Num() != Collection.Transform.Num())
	{
		return 0.0f;
	}
	for (int32 BoneIdx = 0, NumBones = Collection.Transform.Num(); BoneIdx < NumBones; ++BoneIdx)
	{
		if (Collection.SimulationType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			Sum += Volumes[BoneIdx];
		}
	}

	return Sum;
}

static double GetMinVolume(double TotalVolume,
	const EFixTinyGeoGeometrySelectionMethod InSelectionMethod,
	const float InMinVolumeCubeRoot,
	const float InRelativeVolume)
{
	double MinVolume = 0.0;
	if (InSelectionMethod == EFixTinyGeoGeometrySelectionMethod::VolumeCubeRoot)
	{
		MinVolume = InMinVolumeCubeRoot * UE::Dataflow::Private::VolDimScale;
		MinVolume = MinVolume * MinVolume * MinVolume;
	}
	else // EGeometrySelectionMethod::RelativeVolume
	{
		MinVolume = FMath::Pow(TotalVolume, 1.0 / 3.0) * InRelativeVolume;
		MinVolume = MinVolume * MinVolume * MinVolume;
	}

	return MinVolume;
}

static bool CollectTargetBones(FGeometryCollection& Collection,
	const TArray<int32>& Selection,
	TArray<int32>& OutSmallIndices,
	TArray<double>& OutVolumes,
	double& OutMinVolume,
	const EFixTinyGeoMergeType InMergeType,
	const bool InOnFractureLevel,
	const EFixTinyGeoGeometrySelectionMethod InSelectionMethod,
	const float InMinVolumeCubeRoot,
	const float InRelativeVolume,
	const EFixTinyGeoUseBoneSelection InUseBoneSelection,
	const bool InOnlyClusters)
{
	const bool bClusterMode = InMergeType == EFixTinyGeoMergeType::MergeClusters;
	const bool bRestrictToLevel = bClusterMode && InOnFractureLevel;
	int32 InFractureLevel = -1;

	FindBoneVolumes(
		Collection,
		TArrayView<int32>(), /*Empty array => use all transforms*/
		OutVolumes,
		UE::Dataflow::Private::VolDimScale,
		bClusterMode
	);

	double TotalVolume = GetTotalVolume(Collection, OutVolumes);
	OutMinVolume = GetMinVolume(TotalVolume, InSelectionMethod, InMinVolumeCubeRoot, InRelativeVolume);

	TArray<int32> FilteredSelection;

	GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(Collection);

	if (InUseBoneSelection == EFixTinyGeoUseBoneSelection::OnlyMergeSelected)
	{
		if (!bClusterMode)
		{
			OutSmallIndices = Selection;
			SelectionFacade.ConvertSelectionToRigidNodes(OutSmallIndices);
		}
		else
		{
			OutSmallIndices = Selection;
		}
		return !OutSmallIndices.IsEmpty();
	}

	FindSmallBones(
		Collection,
		TArrayView<int32>(),
		OutVolumes,
		OutMinVolume,
		OutSmallIndices,
		bClusterMode
	);

	// Filter bones that aren't at the target level
	const int32 TargetLevel = InFractureLevel;
	const TManagedArray<int32>* LevelAttrib = Collection.FindAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	bool bFilterByLevel = bRestrictToLevel && TargetLevel > -1 && LevelAttrib;
	if (bFilterByLevel)
	{
		OutSmallIndices.SetNum(Algo::RemoveIf(OutSmallIndices, [&](int32 BoneIdx)
			{
				return (*LevelAttrib)[BoneIdx] != TargetLevel;
			}));
	}
	// Filter bones that aren't clusters
	if (bClusterMode && (!bFilterByLevel || InOnlyClusters))
	{
		OutSmallIndices.SetNum(Algo::RemoveIf(OutSmallIndices, [&](int32 BoneIdx)
			{
				return Collection.SimulationType[BoneIdx] != FGeometryCollection::ESimulationTypes::FST_Clustered;
			}));
	}

	if (InUseBoneSelection == EFixTinyGeoUseBoneSelection::AlsoMergeSelected)
	{
		TArray<int32> ProcessedSelection = Selection;
		if (!bClusterMode)
		{
			SelectionFacade.ConvertSelectionToRigidNodes(ProcessedSelection);
		}
		for (int32 Bone : ProcessedSelection)
		{
			OutSmallIndices.AddUnique(Bone);
		}
	}

	return !OutSmallIndices.IsEmpty();
}

void FFractureEngineUtility::FixTinyGeo(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	const EFixTinyGeoMergeType InMergeType,
	const bool InOnFractureLevel,
	const EFixTinyGeoGeometrySelectionMethod InSelectionMethod,
	const float InMinVolumeCubeRoot,
	const float InRelativeVolume,
	const EFixTinyGeoUseBoneSelection InUseBoneSelection,
	const bool InOnlyClusters,
	const EFixTinyGeoNeighborSelectionMethod InNeighborSelection,
	const bool InOnlyToConnected,
	const bool InOnlySameParent,
	const bool bUseCollectionProximity)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		const TArray<int32> TransformSelectionArr = InTransformSelection.AsArray();

		TArray<int32> SmallIndices;
		TArray<double> Volumes;
		double MinVolume;
		if (CollectTargetBones(*GeomCollection,
			TransformSelectionArr,
			SmallIndices,
			Volumes,
			MinVolume,
			InMergeType,
			InOnFractureLevel,
			InSelectionMethod,
			InMinVolumeCubeRoot,
			InRelativeVolume,
			InUseBoneSelection,
			InOnlyClusters))
		{
			// convert ENeighborSelectionMethod to the PlanarCut module version (the only difference is it's not a UENUM)
			UE::PlanarCut::ENeighborSelectionMethod SelectionMethod = UE::PlanarCut::ENeighborSelectionMethod::LargestNeighbor;
			if (InNeighborSelection == EFixTinyGeoNeighborSelectionMethod::NearestCenter)
			{
				SelectionMethod = UE::PlanarCut::ENeighborSelectionMethod::NearestCenter;
			}
			else if (InNeighborSelection == EFixTinyGeoNeighborSelectionMethod::LargestContactArea)
			{
				SelectionMethod = UE::PlanarCut::ENeighborSelectionMethod::LargestContactArea;
			}

			// Make sure we have non-stale proximity data if we will use it
			if (bUseCollectionProximity)
			{
				FGeometryCollectionProximityUtility ProximityUtility(GeomCollection.Get());
				ProximityUtility.UpdateProximity();
			}

			if (InMergeType == EFixTinyGeoMergeType::MergeGeometry)
			{
				MergeBones(*GeomCollection,
					TArrayView<const int32>(), // empty view == consider all bones
					Volumes,
					MinVolume,
					SmallIndices,
					false, /*bUnionJoinedPieces*/ // Note: Union-ing the pieces is nicer in theory, but can leave cracks and non-manifold garbage
					SelectionMethod,
					bUseCollectionProximity,
					InOnlySameParent);
			}
			else
			{
				MergeClusters(*GeomCollection,
					Volumes,
					MinVolume,
					SmallIndices,
					SelectionMethod,
					InOnlyToConnected,
					InOnlySameParent,
					bUseCollectionProximity);
			}

			InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);
		}
	}
}

void FFractureEngineUtility::RecomputeNormalsInGeometryCollection(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	const bool InOnlyTangents,
	const bool InRecomputeSharpEdges,
	const float InSharpEdgeAngleThreshold,
	const bool InOnlyInternalSurfaces)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		TArray<int32> TransformSelectionArr = InTransformSelection.AsArray();

		RecomputeNormalsAndTangents(InOnlyTangents, InRecomputeSharpEdges, InSharpEdgeAngleThreshold, *GeomCollection, TransformSelectionArr, InOnlyInternalSurfaces);

		InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);
	}
}

int32 FFractureEngineUtility::ResampleGeometryCollection(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	const float InCollisionSampleSpacing)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		TArray<int32> TransformSelectionArr = InTransformSelection.AsArray();

		AddCollisionSampleVertices(InCollisionSampleSpacing, *GeomCollection, TransformSelectionArr);

		InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);
	}

	return INDEX_NONE;
}

static void AddSingleRootNodeIfRequired(TUniquePtr<FGeometryCollection>& InOutCollection)
{
	if (InOutCollection)
	{
		if (FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(InOutCollection.Get()))
		{
			FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(InOutCollection.Get());
		}
	}
}

DECLARE_LOG_CATEGORY_EXTERN(LogFractureEngineUtility, Log, All);
DEFINE_LOG_CATEGORY(LogFractureEngineUtility);

void FFractureEngineUtility::ValidateGeometryCollection(FManagedArrayCollection& InOutCollection,
	const bool InRemoveUnreferencedGeometry,
	const bool InRemoveClustersOfOne,
	const bool InRemoveDanglingClusters)
{
	if (TUniquePtr<FGeometryCollection> GeometryCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		bool bDirty = false;

		TManagedArray<int32>& TransformToGeometry = GeometryCollection->TransformToGeometryIndex;
		constexpr bool bClustersCanHaveGeometry = true;
		if (!bClustersCanHaveGeometry)
		{
			// Optionally ensure that clusters do not point to geometry (currently disabled; keeping the geometry can be useful)
			const int32 ElementCount = TransformToGeometry.Num();
			for (int32 Idx = 0; Idx < ElementCount; ++Idx)
			{
				if (GeometryCollection->IsClustered(Idx) && TransformToGeometry[Idx] != INDEX_NONE)
				{
					TransformToGeometry[Idx] = INDEX_NONE;
					UE_LOGF(LogFractureEngineUtility, Verbose, "Removed geometry index from cluster %d.", Idx);
					bDirty = true;
				}
			}
		}

		// Remove any unreferenced geometry
		if (InRemoveUnreferencedGeometry)
		{
			TManagedArray<int32>& TransformIndex = GeometryCollection->TransformIndex;
			const int32 GeometryCount = TransformIndex.Num();

			TArray<int32> RemoveGeometry;
			RemoveGeometry.Reserve(GeometryCount);

			for (int32 Idx = 0; Idx < GeometryCount; ++Idx)
			{
				if ((TransformIndex[Idx] == INDEX_NONE) || (TransformToGeometry[TransformIndex[Idx]] != Idx))
				{
					RemoveGeometry.Add(Idx);
					UE_LOGF(LogFractureEngineUtility, Verbose, "Removed dangling geometry at index %d.", Idx);
					bDirty = true;
				}
			}

			if (RemoveGeometry.Num() > 0)
			{
				FManagedArrayCollection::FProcessingParameters Params;
				Params.bDoValidation = false; // for perf reasons
				GeometryCollection->RemoveElements(FGeometryCollection::GeometryGroup, RemoveGeometry);
			}
		}

		if (InRemoveClustersOfOne)
		{
			if (FGeometryCollectionClusteringUtility::RemoveClustersOfOnlyOneChild(GeometryCollection.Get()))
			{
				UE_LOGF(LogFractureEngineUtility, Verbose, "Removed one or more clusters of only one child.");
				bDirty = true;
			}
		}

		if (InRemoveDanglingClusters)
		{
			if (FGeometryCollectionClusteringUtility::RemoveDanglingClusters(GeometryCollection.Get()))
			{
				UE_LOGF(LogFractureEngineUtility, Verbose, "Removed one or more dangling clusters.");
				bDirty = true;
			}
		}

		if (bDirty)
		{
			FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(*GeometryCollection, -1);
			AddSingleRootNodeIfRequired(GeometryCollection);
		}

		InOutCollection = (const FManagedArrayCollection&)(*GeometryCollection);
	}
}

void FFractureEngineUtility::SplitIslands(FManagedArrayCollection& InOutCollection,
	FDataflowTransformSelection InTransformSelection,
	const float InCloseVertexDistance,
	const float InVertexToSurfaceBridgeDistance)
{
	if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InOutCollection.NewCopy<FGeometryCollection>()))
	{
		TArray<int32> SelectionArray = InTransformSelection.AsArray();
		GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(*GeomCollection);
		SelectionFacade.ConvertSelectionToRigidNodes(SelectionArray);

		FIslandSplitSettings Settings(true, (double)InCloseVertexDistance, (double)InVertexToSurfaceBridgeDistance);

		::SplitIslands(
			*GeomCollection,
			SelectionArray,
			0.0,
			nullptr,
			Settings);

		InOutCollection = (const FManagedArrayCollection&)(*GeomCollection);
	}
}

