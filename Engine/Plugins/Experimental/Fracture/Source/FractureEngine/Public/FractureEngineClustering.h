// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Count.h"
#include "Math/Box.h"

#define UE_API FRACTUREENGINE_API

class FGeometryCollection;


class FVoronoiPartitioner
{
public:
	UE_API FVoronoiPartitioner(const FGeometryCollection* GeometryCollection, int32 ClusterIndex);

	/** 
	 * Cluster bodies into k partitions using K-Means. Connectivity is ignored: only spatial proximity is considered. 
	 * @param InPartitionCount	Number of partitions to target, if InitialCenters is not provided
	 * @param MaxIterations		Maximum iterations of refinement of partitions. In many cases, K-Means will converge and stop early if MaxIterations is large.
	 * @param InitialCenters	If non-empty, these positions will be used to initialize the partition locations. The target partition count will then be the length of this array.
	 */
	UE_API void KMeansPartition(int32 InPartitionCount, int32 MaxIterations = 500, TArrayView<const FVector> InitialCenters = TArrayView<const FVector>());

	/** Split any partition islands into their own partition. This will possbily increase number of partitions to exceed desired count. */
	UE_API void SplitDisconnectedPartitions(FGeometryCollection* GeometryCollection);

	/** Merge any partitions w/ only 1 body into a connected, neighboring partition (if any).  This can decrease the number of partitions below the desired count. */
	UE_API void MergeSingleElementPartitions(FGeometryCollection* GeometryCollection);

	/** Merge any too-small partitions into a connected, neighboring partition (if any).  This can decrease the number of partitions below the desired count. */
	UE_API void MergeSmallPartitions(FGeometryCollection* GeometryCollection, float PartitionSizeThreshold);

	int32 GetPartitionCount() const { return PartitionCount; }

	int32 GetNonEmptyPartitionCount() const
	{
		return PartitionSize.Num() - Algo::Count(PartitionSize, 0);
	}

	int32 GetIsolatedPartitionCount() const
	{
		return Algo::Count(PartitionSize, 1);
	}

	/** return the GeometryCollection TranformIndices within the partition. */
	UE_API TArray<int32> GetPartition(int32 PartitionIndex) const;

	static UE_API FBox GenerateBounds(const FGeometryCollection* GeometryCollection, int32 TransformIndex);

private:
	UE_API void GenerateConnectivity(const FGeometryCollection* GeometryCollection);
	UE_API void CollectConnections(const FGeometryCollection* GeometryCollection, int32 Index, int32 OperatingLevel, TSet<int32>& OutConnections) const;
	UE_API void GenerateCentroids(const FGeometryCollection* GeometryCollection);
	UE_API FVector GenerateCentroid(const FGeometryCollection* GeometryCollection, int32 TransformIndex) const;
	UE_API void InitializePartitions(TArrayView<const FVector> InitialCenters = TArrayView<const FVector>());
	UE_API bool Refine();
	UE_API int32 FindClosestPartitionCenter(const FVector& Location) const;
	UE_API void MarkVisited(int32 Index, int32 PartitionIndex);

private:
	TArray<int32> TransformIndices;
	TArray<FVector> Centroids;
	// mapping from index into TransformIndices to partition number
	TArray<int32> Partitions;
	int32 PartitionCount;
	TArray<int32> PartitionSize;
	TArray<FVector> PartitionCenters;
	// mapping from index into TransformIndices to the set of connected transforms (also via their index in TransformIndices)
	TArray<TSet<int32>> Connectivity;
	TArray<bool> Visited;
};

enum class EFractureEngineClusterSizeMethod : uint8
{
	// Cluster by specifying an absolute number of clusters
	ByNumber,
	// Cluster by specifying a fraction of the number of input bones
	ByFractionOfInput,
	// Cluster by specifying the density of the input bones
	BySize,
	// Cluster by a regular grid distribution
	ByGrid,
};

class FFractureEngineClustering
{
public:

	static UE_API void AutoCluster(FGeometryCollection& GeometryCollection,
		const TArray<int32>& BoneIndices,
		const EFractureEngineClusterSizeMethod ClusterSizeMethod,
		const uint32 SiteCount,
		const float SiteCountFraction,
		const float SiteSize,
		const bool bEnforceConnectivity,
		const bool bAvoidIsolated,
		const bool bEnforceSiteParameters,
		const int32 GridX = 2,
		const int32 GridY = 2,
		const int32 GridZ = 2,
		const float MinimumClusterSize = 0,
		const int32 KMeansIterations = 500,
		const bool bPreferConvexity = false,
		const float ConcavityErrorTolerance = 0);

	static UE_API void AutoCluster(FGeometryCollection& GeometryCollection,
		const int32 ClusterIndex,
		const EFractureEngineClusterSizeMethod ClusterSizeMethod,
		const uint32 SiteCount,
		const float SiteCountFraction,
		const float SiteSize,
		const bool bEnforceConnectivity,
		const bool bAvoidIsolated,
		const bool bEnforceSiteParameters,
		const int32 GridX = 2,
		const int32 GridY = 2,
		const int32 GridZ = 2,
		const float MinimumClusterSize = 0,
		const int32 KMeansIterations = 500,
		const bool bPreferConvexity = false,
		const float ConcavityErrorTolerance = 0);

	// Autoclustering that favors convex-shaped clusters
	static UE_API void ConvexityBasedCluster(FGeometryCollection& GeometryCollection,
		int32 ClusterIndex,
		uint32 SiteCount,
		bool bEnforceConnectivity,
		bool bAvoidIsolated,
		float ConcavityErrorTolerance);

	static UE_API TArray<FVector> GenerateGridSites(
		const FGeometryCollection& GeometryCollection,
		const TArray<int32>& BoneIndices,
		const int32 GridX,
		const int32 GridY,
		const int32 GridZ);

	static UE_API TArray<FVector> GenerateGridSites(
		const FGeometryCollection& GeometryCollection,
		const int32 ClusterIndex,
		const int32 GridX,
		const int32 GridY,
		const int32 GridZ,
		FBox* OutBounds = nullptr);

	// Cluster the chosen transform indices (and update the selection array to remove any that were not clustered, i.e. invalid or root transforms)
	// @return true if the GeometryCollection was updated
	static UE_API bool ClusterSelected(
		FGeometryCollection& GeometryCollection,
		TArray<int32>& InOutSelection
	);

	// Merge selected clusters. Non-clusters in the selection are converted to the closest (parent) clusters.
	// On success, returns true and InOutSelection holds the index of the cluster to which the selection was merged.
	static UE_API bool MergeSelectedClusters(
		FGeometryCollection& GeometryCollection,
		TArray<int32>& InOutSelection
	);

	// Merge neighbors, and neighbors of neighbors (out to the Iterations number) to the selected clusters.
	static UE_API bool ClusterMagnet(
		FGeometryCollection& GeometryCollection,
		TArray<int32>& InOutSelection,
		int32 Iterations
	);
};

#undef UE_API
