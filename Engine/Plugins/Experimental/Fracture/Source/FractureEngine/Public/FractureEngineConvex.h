// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CompGeom/ConvexDecomposition3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UObject/ObjectMacros.h"
#include "FractureEngineConvex.generated.h"

namespace Chaos { class FConvex; }
struct FManagedArrayCollection;

UENUM()
enum class EConvexHullSimplifyMethod
{
	MeshQSlim,
	AngleTolerance
};

namespace UE::FractureEngine::Convex
{
	// Options to control the simplification of an existing convex hull
	struct FSimplifyHullSettings
	{
		EConvexHullSimplifyMethod SimplifyMethod = EConvexHullSimplifyMethod::MeshQSlim;

		double AngleThreshold = 0; // used for AngleTolerance simplification
		double SmallAreaThreshold = .1; // optionally used for AngleTolerance to ignore very small faces

		bool bUseGeometricTolerance = true;
		double ErrorTolerance = 5;

		bool bUseTargetTriangleCount = false;
		int32 TargetTriangleCount = 20;

		bool bUseExistingVertexPositions = true;
	};

	/**
	 * Get the convex hulls on the given Collection as a Dynamic Mesh. Optionally only include the hulls on the transforms in TransformSelection.
	 * @return true if the collection has convex hulls, the bone selection (if present) was valid, and all requested convex hulls were included in the mesh
	 */
	bool FRACTUREENGINE_API GetConvexHullsAsDynamicMesh(const FManagedArrayCollection& Collection, UE::Geometry::FDynamicMesh3& UpdateMesh, bool bRestrictToSelection = false, const TArrayView<const int32> TransformSelection = TArrayView<const int32>(), bool bRecomputeHulls = false);

	/**
	* Get the convex hulls on the given Collection as an array of dynamic meshes. Optionally only include the hulls on the transforms in TransformSelection.
	* @return true if the collection has convex hulls, the bone selection (if present) was valid, and all requested convex hulls were included in the mesh array 
	*/
	bool FRACTUREENGINE_API GetConvexHullsAsDynamicMeshes(const FManagedArrayCollection& Collection, TArray<UE::Geometry::FDynamicMesh3>& UpdateMeshes, bool bRestrictToSelection = false, const TArrayView<const int32> TransformSelection = TArrayView<const int32>(), bool bRecomputeHulls = false);

	/**
	 * Simplify the convex hulls on the given Collection. Optionally only simplify the hulls on the transforms in TransformSelection.
	 * @return true if the collection has convex hulls and the hulls were either simplified or did not need to be
	 */
	bool FRACTUREENGINE_API SimplifyConvexHulls(FManagedArrayCollection& Collection, const FSimplifyHullSettings& Settings, bool bRestrictToSelection = false, const TArrayView<const int32> TransformSelection = TArrayView<const int32>());

	/**
	 * Simplify a convex hull using the given Settings.OutConvexHull can optionally be the same pointer as InConvexHull.
	 * @return true if the hull had valid data and either was simplified or did not need to be (e.g. already had few enough triangles)
	 */
	bool FRACTUREENGINE_API SimplifyConvexHull(const ::Chaos::FConvex* InConvexHull, ::Chaos::FConvex* OutConvexHull, const FSimplifyHullSettings& Settings);

	/**
	 * Compute negative space for the convex hulls on a geometry collection
	 * @return true if any negative space was found
	 */
	bool FRACTUREENGINE_API ComputeConvexHullsNegativeSpace(FManagedArrayCollection& Collection, UE::Geometry::FSphereCovering& OutNegativeSpace, const UE::Geometry::FNegativeSpaceSampleSettings& Settings, bool bRestrictToSelection = false, const TArrayView<const int32> TransformSelection = TArrayView<const int32>(), bool bFromRigidTransforms = true);

}

