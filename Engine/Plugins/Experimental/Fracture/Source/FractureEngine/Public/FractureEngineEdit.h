// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h" // IWYU pragma: keep

#define UE_API FRACTUREENGINE_API

class FGeometryCollection;
struct FManagedArrayCollection;

class FFractureEngineEdit
{
public:

	static UE_API void DeleteBranch(FGeometryCollection& GeometryCollection, const TArray<int32>& InBoneSelection);

	static UE_API void SetVisibilityInCollectionFromTransformSelection(FManagedArrayCollection& InCollection, const TArray<int32>& InTransformSelection, bool bVisible);

	static UE_API void SetVisibilityInCollectionFromFaceSelection(FManagedArrayCollection& InCollection, const TArray<int32>& InFaceSelection, bool bVisible);

	static UE_API void Merge(FGeometryCollection& GeometryCollection, const TArray<int32>& InBoneSelection);

};

#undef UE_API
