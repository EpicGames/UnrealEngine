// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "PlanarCut.h"
#include "Dataflow/DataflowSelection.h"

#include "FractureEngineFracturing.generated.h"

#define UE_API FRACTUREENGINE_API

namespace UE { namespace Geometry { class FDynamicMesh3; } }

struct FManagedArrayCollection;
class FGeometryCollection;

UENUM(BlueprintType)
enum class EFractureBrickBondEnum : uint8
{
	Dataflow_FractureBrickBond_Stretcher UMETA(DisplayName = "Stretcher"),
	Dataflow_FractureBrickBond_Stack UMETA(DisplayName = "Stack"),
	Dataflow_FractureBrickBond_English UMETA(DisplayName = "English"),
	Dataflow_FractureBrickBond_Header UMETA(DisplayName = "Header"),
	Dataflow_FractureBrickBond_Flemish UMETA(DisplayName = "Flemish"),
};

UENUM(BlueprintType)
enum class EMeshCutterCutDistribution : uint8
{
	// Cut only once, at the cutting mesh's current location in the level
	SingleCut UMETA(DisplayName = "Single Cut"),
	// Scatter the cutting mesh in a uniform random distribution around the geometry bounding box
	UniformRandom UMETA(DisplayName = "Uniform Random"),
	// Arrange the cutting mesh in a regular grid pattern
	Grid UMETA(DisplayName = "Grid"),
};

UENUM(BlueprintType)
enum class EMeshCutterPerCutMeshSelection : uint8
{
	// Use all cut meshes for every cut
	All,
	// Choose which cut mesh to use from the mesh array randomly, per cut
	Random,
	// Choose which cut mesh to use from the mesh array sequentially, starting with the first mesh and cycling through, per cut
	Sequential,
};

struct FUniformFractureSettings
{
	FTransform Transform;
	int32 MinVoronoiSites;
	int32 MaxVoronoiSites;
	int32 InternalMaterialID;
	int32 RandomSeed;
	float ChanceToFracture;
	bool GroupFracture;
	bool SplitIslands;
	double CloseVertexDistance = 1e-3;
	double VertexToSurfaceBridgeDistance = 0;
	float Grout;
	FNoiseSettings NoiseSettings;
	bool AddSamplesForCollision;
	float CollisionSampleSpacing;
};

struct FUniformFractureProcSettings
{
	FBox BBox;
	FTransform Transform;
	int32 MinVoronoiSites;
	int32 MaxVoronoiSites;
	int32 InternalMaterialID;
	int32 RandomSeed;
	bool SplitIslands;
	double CloseVertexDistance = 1e-3;
	double VertexToSurfaceBridgeDistance = 0;
	float Grout;
	FNoiseSettings NoiseSettings;
	bool AddSamplesForCollision;
	float CollisionSampleSpacing;
};

class FFractureEngineFracturing
{
public:
	static UE_API void GenerateExplodedViewAttribute(
		FManagedArrayCollection& InOutCollection,
		const FVector& InScale,
		const float InUniformScale,
		const int32 InViewFractureLevel = -1,
		const int32 InMaxFractureLevel = -1);

	static UE_API int32 VoronoiFracture(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		TArray<FVector> InSites,
		const FTransform& InTransform,
		int32 InRandomSeed,
		float InChanceToFracture,
		FIslandSplitSettings IslandSplitSettings,
		float InGrout,
		float InAmplitude,
		float InFrequency,
		float InPersistence,
		float InLacunarity,
		int32 InOctaveNumber,
		float InPointSpacing,
		bool InAddSamplesForCollision,
		float InCollisionSampleSpacing);

	static UE_API void GenerateSliceTransforms(
		const FBox& InBoundingBox,
		const int32 InRandomSeed,
		const int32 InNumPlanes,
		TArray<FTransform>& OutCuttingPlaneTransforms);
	
	static UE_API int32 PlaneCutter(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const FBox& InBoundingBox,
		const FTransform& InTransform,
		int32 InNumPlanes,
		int32 InRandomSeed,
		float InChanceToFracture,
		FIslandSplitSettings IslandSplitSettings,
		float InGrout,
		float InAmplitude,
		float InFrequency,
		float InPersistence,
		float InLacunarity,
		int32 InOctaveNumber,
		float InPointSpacing,
		bool InAddSamplesForCollision,
		float InCollisionSampleSpacing,
		TConstArrayView<FTransform> InCutPlaneTransforms = TConstArrayView<FTransform>());

	static UE_API void GenerateSliceTransforms(TArray<FTransform>& InOutCuttingPlaneTransforms,
		const FBox& InBoundingBox,
		int32 InSlicesX,
		int32 InSlicesY,
		int32 InSlicesZ,
		int32 InRandomSeed,
		float InSliceAngleVariation,
		float InSliceOffsetVariation);

	static UE_API int32 SliceCutter(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const FBox& InBoundingBox,
		int32 InSlicesX,
		int32 InSlicesY,
		int32 InSlicesZ,
		float InSliceAngleVariation,
		float InSliceOffsetVariation,
		int32 InRandomSeed,
		float InChanceToFracture,
		FIslandSplitSettings IslandSplitSettings,
		float InGrout,
		float InAmplitude,
		float InFrequency,
		float InPersistence,
		float InLacunarity,
		int32 InOctaveNumber,
		float InPointSpacing,
		bool InAddSamplesForCollision,
		float InCollisionSampleSpacing);

	static UE_API void AddBoxEdges(TArray<TTuple<FVector, FVector>>& InOutEdges, 
		const FVector& InMin, 
		const FVector& InMax);

	static UE_API void GenerateBrickTransforms(const FBox& InBounds,
		TArray<FTransform>& InOutBrickTransforms,
		const EFractureBrickBondEnum InBond,
		const float InBrickLength,
		const float InBrickHeight,
		const float InBrickDepth,
		TArray<TTuple<FVector, FVector>>& InOutEdges);

	static UE_API int32 BrickCutter(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const FBox& InBoundingBox, 
		const FTransform& InTransform,
		EFractureBrickBondEnum InBond,
		float InBrickLength,
		float InBrickHeight,
		float InBrickDepth,
		int32 InRandomSeed,
		float InChanceToFracture,
		FIslandSplitSettings IslandSplitSettings,
		float InGrout,
		float InAmplitude,
		float InFrequency,
		float InPersistence,
		float InLacunarity,
		int32 InOctaveNumber,
		float InPointSpacing,
		bool InAddSamplesForCollision,
		float InCollisionSampleSpacing);

	static UE_API void GenerateMeshTransforms(TArray<FTransform>& MeshTransforms,
		const FBox& InBoundingBox,
		const int32 InRandomSeed,
		const EMeshCutterCutDistribution InCutDistribution,
		const int32 InNumberToScatter,
		const int32 InGridX,
		const int32 InGridY,
		const int32 InGridZ,
		const float InVariability,
		const float InMinScaleFactor,
		const float InMaxScaleFactor,
		const bool InRandomOrientation,
		const float InRollRange,
		const float InPitchRange,
		const float InYawRange);

	static UE_API int32 MeshCutter(TArray<FTransform>& MeshTransforms,
		FManagedArrayCollection& InOutCollection,
		const FDataflowTransformSelection& InTransformSelection,
		const UE::Geometry::FDynamicMesh3& InDynCuttingMesh,
		const int32 InRandomSeed,
		const float InChanceToFracture,
		const FIslandSplitSettings IslandSplitSettings,
		const float InCollisionSampleSpacing);

	static UE_API int32 MeshArrayCutter(TArray<FTransform>& MeshTransforms,
		FManagedArrayCollection& InOutCollection,
		const FDataflowTransformSelection& InTransformSelection,
		TArrayView<const UE::Geometry::FDynamicMesh3*> InDynCuttingMeshes,
		const EMeshCutterPerCutMeshSelection PerCutMeshSelection,
		const int32 InRandomSeed,
		const float InChanceToFracture,
		const FIslandSplitSettings IslandSplitSettings,
		const float InCollisionSampleSpacing);

	static UE_API int32 UniformFracture(
		FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const FUniformFractureSettings& InUniformFractureSettings);

	static UE_API void InitColors(FManagedArrayCollection& InCollection);

	static UE_API void TransferBoneColorToVertexColor(FManagedArrayCollection& InCollection);

	static UE_API void SetBoneColorByParent(
		FManagedArrayCollection& InCollection,
		const FRandomStream& InRandomStream,
		int32 InLevel,
		const int32 InColorRangeMin = 40,
		const int32 InColorRangeMax = 190);

	static UE_API void SetBoneColorByLevel(
		FManagedArrayCollection& InCollection, 
		int32 InLevel);

	static UE_API void SetBoneColorByCluster(
		FManagedArrayCollection& InCollection,
		const FRandomStream& InRandomStream,
		int32 InLevel,
		const int32 InColorRangeMin,
		const int32 InColorRangeMax);

	static UE_API void SetBoneColorByLeafLevel(
		FManagedArrayCollection& InCollection, 
		int32 InLevel);

	static UE_API void SetBoneColorByLeaf(
		FManagedArrayCollection& InCollection, 
		const FRandomStream& InRandomStream,
		int32 InLevel,
		const int32 InColorRangeMin,
		const int32 InColorRangeMax);

	static UE_API void SetBoneColorByAttr(
		FManagedArrayCollection& InCollection,
		const FString InAttribute,
		const float InMinAttrValue,
		float InMaxAttrValue,
		const FLinearColor InMinColor,
		const FLinearColor InMaxColor);

	static UE_API void SetBoneColorRandom(
		FManagedArrayCollection& InCollection,
		const FRandomStream& InRandomStream);
};
	

#undef UE_API
