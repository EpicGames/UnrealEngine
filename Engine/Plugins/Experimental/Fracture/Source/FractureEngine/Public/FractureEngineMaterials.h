// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h" // IWYU pragma: keep

#define UE_API FRACTUREENGINE_API

class FGeometryCollection;
struct FManagedArrayCollection;

class FFractureEngineMaterials
{
public:

	enum class ETargetFaces
	{
		InternalFaces,
		ExternalFaces,
		AllFaces
	};

	static UE_API void SetMaterial(FManagedArrayCollection& InCollection, const TArray<int32>& InBoneSelection, ETargetFaces TargetFaces, int32 MaterialID);

	static UE_API void SetMaterialOnGeometryAfter(FManagedArrayCollection& InCollection, int32 FirstGeometryIndex, ETargetFaces TargetFaces, int32 MaterialID);

	static void SetMaterialOnAllGeometry(FManagedArrayCollection& InCollection, ETargetFaces TargetFaces, int32 MaterialID)
	{
		SetMaterialOnGeometryAfter(InCollection, 0, TargetFaces, MaterialID);
	}

};

#undef UE_API
