// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "Sampling/MeshSurfacePointSampling.h"

#include "FractureEngineSampling.generated.h"

namespace UE::Geometry { class FDynamicMesh3; }

struct FDataflowTransformSelection;
struct FManagedArrayCollection;

UENUM(BlueprintType)
enum class ENonUniformSamplingDistributionMode : uint8
{
	/** Uniform distribution of sizes, ie all equally likely */
	ENonUniformSamplingDistributionMode_Uniform UMETA(DisplayName = "Uniform"),
	/** Distribution is weighted towards smaller points */
	ENonUniformSamplingDistributionMode_Smaller UMETA(DisplayName = "Smaller"),
	/** Distribution is weighted towards larger points */
	ENonUniformSamplingDistributionMode_Larger UMETA(DisplayName = "Larger"),
};

UENUM(BlueprintType)
enum class ENonUniformSamplingWeightMode : uint8
{
	/**
	 * Weights are clamped to [0,1] and used to interpolate Min/Max Radius. This is a "hard constraint", ie if the weight
	 * at a point is 1, only a "max radius" sample may be placed there, otherwise no samples at all (so no "filling in" smaller samples between large ones)
	 */
	ENonUniformSamplingWeightMode_WeightToRadius UMETA(DisplayName = "Weight To Radius"),
	/**
	 * Weights are clamped to [0,1] and used to interpolate Min/Max Radius, with decay, so that smaller-radius samples will infill between large ones.
	 * So areas with large weight may still end up with some variable-radius samples, but areas with 0 weight will only ever have min-radius samples.
	 */
	ENonUniformSamplingWeightMode_FilledWeightToRadius UMETA(DisplayName = "Filled Weight To Radius"),
	/**
	 * Weight is used to create nonuniform random sampling, ie it nudges the random sample-radius distribution but does not directly control it.
	 * So samples with any radius can still appear at any location, but if weight=1 then max-radius samples are more likely, etc.
	 */
	ENonUniformSamplingWeightMode_WeightedRandom UMETA(DisplayName = "Weighted Random"),
};

class FFractureEngineSampling
{
public:
	/**
	* Computes a Poisson sampling on the InMesh
	*/
	FRACTUREENGINE_API static void ComputeUniformPointSampling(const UE::Geometry::FDynamicMesh3& InMesh,
		const float InSamplingRadius,
		const int32 InMaxNumSamples,
		const float InSubSampleDensity,
		const int32 InRandomSeed,
		TArray<FTransform>& OutSamples,
		TArray<int32>& OutTriangleIDs,
		TArray<FVector>& OutBarycentricCoords);

	/**
	* Computes a NonUniform sampling on the InMesh
	*/
	FRACTUREENGINE_API static void ComputeNonUniformPointSampling(const UE::Geometry::FDynamicMesh3& InMesh,
		const float InSamplingRadius,
		const int32 InMaxNumSamples,
		const float InSubSampleDensity,
		const int32 InRandomSeed,
		const float InMaxSamplingRadius,
		const ENonUniformSamplingDistributionMode InSizeDistribution,
		const float InSizeDistributionPower,
		TArray<FTransform>& OutSamples,
		TArray<float>& OutSampleRadii,
		TArray<int32>& OutTriangleIDs,
		TArray<FVector>& OutBarycentricCoords);

	/**
	* Computes a vertex weighted sampling on the InMesh
	*/
	FRACTUREENGINE_API static void ComputeVertexWeightedPointSampling(const UE::Geometry::FDynamicMesh3& InMesh,
		const TArray<float>& InVertexWeights,
		const float InSamplingRadius,
		const int32 InMaxNumSamples,
		const float InSubSampleDensity,
		const int32 InRandomSeed,
		const float InMaxSamplingRadius,
		const ENonUniformSamplingDistributionMode InSizeDistribution,
		const float InSizeDistributionPower,
		const ENonUniformSamplingWeightMode InWeightMode,
		const bool InInvertWeights,
		TArray<FTransform>& OutSamples,
		TArray<float>& OutSampleRadii,
		TArray<int32>& OutTriangleIDs,
		TArray<FVector>& OutBarycentricCoords);
};


