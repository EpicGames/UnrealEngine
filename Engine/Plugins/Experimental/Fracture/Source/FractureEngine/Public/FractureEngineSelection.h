// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h" // IWYU pragma: keep

#define UE_API FRACTUREENGINE_API

class FGeometryCollection;
struct FDataflowTransformSelection;
struct FManagedArrayCollection;
class FName;

class FFractureEngineSelection
{
public:
	static UE_API void GetRootBones(const FManagedArrayCollection& Collection, TArray<int32>& RootBonesOut);
	static UE_API void GetRootBones(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectParent(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones);
	static UE_API void SelectParent(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectChildren(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones);
	static UE_API void SelectChildren(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectSiblings(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones);
	static UE_API void SelectSiblings(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectLevel(const FManagedArrayCollection& Collection, TArray<int32>& SelectedBones);
	static UE_API void SelectLevel(const FManagedArrayCollection& Collection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectContact(FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones);
	static UE_API void SelectContact(FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectLeaf(const FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones);
	static UE_API void SelectLeaf(const FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectCluster(const FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones);
	static UE_API void SelectCluster(const FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection);

	static UE_API void SelectByPercentage(TArray<int32>& SelectedBones, const int32 Percentage, const bool Deterministic, const float RandomSeed);
	static UE_API void SelectByPercentage(FDataflowTransformSelection& TransformSelection, const int32 Percentage, const bool Deterministic, const float RandomSeed);

	static UE_API void SelectBySize(FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones, const float SizeMin, const float SizeMax);
	static UE_API void SelectBySize(FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection, const float SizeMin, const float SizeMax);

	static UE_API void SelectByVolume(FGeometryCollection& GeometryCollection, TArray<int32>& SelectedBones, const float VolumeMin, const float VolumeMax);
	static UE_API void SelectByVolume(FGeometryCollection& GeometryCollection, FDataflowTransformSelection& TransformSelection, const float VolumeMin, const float VolumeMax);

	static UE_API bool IsBoneSelectionValid(const FManagedArrayCollection& Collection, const TArray<int32>& SelectedBones);
	static UE_API bool IsSelectionValid(const FManagedArrayCollection& Collection, const TArray<int32>& SelectedItems, const FName ItemGroup);
};

#undef UE_API
