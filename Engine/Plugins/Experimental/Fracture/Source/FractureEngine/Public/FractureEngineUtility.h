// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"

#include "FractureEngineUtility.generated.h"

#define UE_API FRACTUREENGINE_API

namespace UE::Geometry { class FDynamicMesh3; }

struct FDataflowTransformSelection;
struct FManagedArrayCollection;

UENUM(BlueprintType)
enum class EFixTinyGeoMergeType : uint8
{
	MergeGeometry UMETA(DisplayName = "Merge Geometry"),
	MergeClusters UMETA(DisplayName = "Merge Clusters"),
};

UENUM(BlueprintType)
enum class EFixTinyGeoNeighborSelectionMethod : uint8
{
	// Merge to the neighbor with the largest volume
	LargestNeighbor UMETA(DisplayName = "Largest Neighbor"),
	// Merge to the neighbor with the closest center
	NearestCenter UMETA(DisplayName = "Nearest Center"),
	// Merge to the neighbor with the largest contact surface area
	LargestContactArea
};

UENUM(BlueprintType)
enum class EFixTinyGeoUseBoneSelection : uint8
{
	NoEffect UMETA(DisplayName = "No Effect"),
	AlsoMergeSelected UMETA(DisplayName = "Also Merge Selected"),
	OnlyMergeSelected UMETA(DisplayName = "Only Merge Selected"),
};

UENUM(BlueprintType)
enum class EFixTinyGeoGeometrySelectionMethod : uint8
{
	// Select by cube root of volume
	VolumeCubeRoot UMETA(DisplayName = "Size"),
	// Select by cube root of volume relative to the overall shape's cube root of volume
	RelativeVolume UMETA(DisplayName = "Relative Volume"),
};

class FFractureEngineUtility
{
public:
	/**
	* Outputs the vertex and triangle data of a FBox into OutVertices and OutTriangles
	*/
	static UE_API void ConvertBoxToVertexAndTriangleData(const FBox& InBox, TArray<FVector3f>& OutVertices, TArray<FIntVector>& OutTriangles);

	/**
	* Creates a mesh from vertex and triangle data
	*/
	static UE_API void ConstructMesh(UE::Geometry::FDynamicMesh3& OutMesh, const TArray<FVector3f>& InVertices, const TArray<FIntVector>& InTriangles);

	/** 
	* Outputs the vertex and triangle data of a mesh into OutVertices and OutTriangles
	*/
	static UE_API void DeconstructMesh(const UE::Geometry::FDynamicMesh3& InMesh, TArray<FVector3f>& OutVertices, TArray<FIntVector>& OutTriangles);

	static UE_API void FixTinyGeo(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const EFixTinyGeoMergeType InMergeType,
		const bool InOnFractureLevel,
		const EFixTinyGeoGeometrySelectionMethod InSelectionMethod,
		const float InMinVolumeCubeRoot,
		const float InRelativeVolume,
		const EFixTinyGeoUseBoneSelection InUseBoneSelection,
		const bool InOnlyClusters,
		const EFixTinyGeoNeighborSelectionMethod InNeighborSelection,
		const bool InOnlyToConnected,
		const bool InOnlySameParent,
		const bool bUseCollectionProximity = true);

	static UE_API void RecomputeNormalsInGeometryCollection(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const bool InOnlyTangents,
		const bool InRecomputeSharpEdges,
		const float InSharpEdgeAngleThreshold,
		const bool InOnlyInternalSurfaces);

	static UE_API int32 ResampleGeometryCollection(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const float InCollisionSampleSpacing);

	static UE_API void ValidateGeometryCollection(FManagedArrayCollection& InOutCollection,
		const bool InRemoveUnreferencedGeometry,
		const bool InRemoveClustersOfOne,
		const bool InRemoveDanglingClusters);

	static UE_API void SplitIslands(FManagedArrayCollection& InOutCollection,
		FDataflowTransformSelection InTransformSelection,
		const float InCloseVertexDistance,
		const float InVertexToSurfaceBridgeDistance);

};

#undef UE_API
